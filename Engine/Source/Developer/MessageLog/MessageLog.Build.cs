// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MessageLog : ModuleRules
{
	public MessageLog(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"Slate",
				"SlateCore",
                "InputCore",
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Documentation",
					"MainFrame",
					"WorkspaceMenuStructure"
				}
			);		
		}

		if (Target.bCompileAgainstEngine && Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrecompileForTargets = PrecompileTargetsType.Any;
		}

		bDisableAutoRTFMInstrumentation = true;
	}
}
