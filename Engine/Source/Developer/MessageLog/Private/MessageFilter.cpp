// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageFilter.h"
#include "Styling/SlateTypes.h"

FReply FMessageFilter::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	RefreshCallback.Broadcast();
	return FReply::Handled();
}

ECheckBoxState FMessageFilter::OnGetDisplayCheckState() const
{
	return Display ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FMessageFilter::OnDisplayCheckStateChanged(ECheckBoxState InNewState)
{
	Display = InNewState == ECheckBoxState::Checked;
	RefreshCallback.Broadcast();
}
