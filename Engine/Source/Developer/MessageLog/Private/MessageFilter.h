// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Textures/SlateIcon.h"

enum class ECheckBoxState : uint8;

/**
 * Represents a message log attribute that can be filtered.
 * Handles all callbacks for any slate checkboxes which wish to alter such filters.
 */
class FMessageFilter : public TSharedFromThis<FMessageFilter>
{
public:
	FMessageFilter(const FText& InName, const FSlateIcon& InIcon)
		: Name(InName)
		, Icon(InIcon)
		, Display(true)
	{}

	virtual ~FMessageFilter() = default;

	/** Fires a callback when the mouse is released on an option (not linked to states) */
	FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Gets the display state to send to a display filter check box */
	virtual ECheckBoxState OnGetDisplayCheckState() const;
	
	/** Sets the display state from a display filter check box */
	virtual void OnDisplayCheckStateChanged(ECheckBoxState InNewState);

	/**
	 * Returns a delegate to be invoked when the filter state changes.
	 *
	 * @return The delegate.
	 */
	virtual FSimpleMulticastDelegate& OnFilterChanged( )
	{
		return RefreshCallback;
	}


	const FText& GetName() const {return Name;}
	const FSlateIcon& GetIcon() const {return Icon;}
	const bool GetDisplay() const {return Display;}

private:
	FText Name;
	FSlateIcon Icon;
	bool Display;

	FSimpleMulticastDelegate RefreshCallback;
};
