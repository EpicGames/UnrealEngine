// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageLogModule.h"
#include "Logging/IMessageLog.h"
#include "IMessageLogListing.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "UserInterface/SMessageLog.h"
#include "Model/MessageLogListingModel.h"
#include "Presentation/MessageLogListingViewModel.h"
#include "UserInterface/SMessageLogListing.h"
#include "Model/MessageLogModel.h"
#include "Presentation/MessageLogViewModel.h"
#include "Logging/MessageLog.h"

#if WITH_EDITOR
	#include "WorkspaceMenuStructure.h"
	#include "WorkspaceMenuStructureModule.h"
#endif

IMPLEMENT_MODULE( FMessageLogModule, MessageLog );

static bool GDisableMessageLogPopup = false;
FAutoConsoleVariableRef CVarDisableMessageLogPopup(
	TEXT("MessageLog.DisablePopup"),
	GDisableMessageLogPopup,
	TEXT("Disables the MessageLog popup"),
	ECVF_Default
);

TSharedRef<SDockTab> SpawnMessageLog( const FSpawnTabArgs& Args, TSharedRef<FMessageLogViewModel> MessageLogViewModel )
{
	LLM_SCOPE(ELLMTag::UI);
	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SMessageLog, MessageLogViewModel)
		];
}

static TSharedRef<IMessageLog> GetLog(const FName& LogName)
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	return MessageLogModule.GetLogListing(LogName);
}

FMessageLogModule::FMessageLogModule()
	: bCanDisplayMessageLog(false)
{
}

void FMessageLogModule::StartupModule()
{
	MessageLogViewModel = MakeShareable(new FMessageLogViewModel(MakeShareable(new FMessageLogModel())));
	MessageLogViewModel->Initialize();

#if WITH_EDITOR
	ModulesChangedHandle = FModuleManager::Get().OnModulesChanged().AddLambda(
	[this](FName InModuleName, EModuleChangeReason InReason)
	{
		if (InReason == EModuleChangeReason::ModuleLoaded && InModuleName == "LevelEditor")
		{
			this->RegisterMessageLogSpawner(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsLogCategory());
		}
	});
#endif

	if (!IsRunningCommandlet()) // Commandlets can just use FBasicMessageLog from Core
	{
		// Bind us so message log output is routed via this module
		FMessageLog::OnGetLog().BindStatic(&GetLog);
	}
}

void FMessageLogModule::ShutdownModule()
{
#if WITH_EDITOR
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner("MessageLog");
	}
	FModuleManager::Get().OnModulesChanged().Remove(ModulesChangedHandle);
#endif

	if (!IsRunningCommandlet()) // Commandlets can just use FBasicMessageLog from Core
	{
		FMessageLog::OnGetLog().Unbind();
	}
}

TSharedRef<IMessageLogListing> FMessageLogModule::GetLogListing(const FName& LogName)
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	return MessageLogViewModel->GetLogListingViewModel(LogName);
}

void FMessageLogModule::RegisterLogListing(const FName& LogName, const FText& LogLabel, const FMessageLogInitializationOptions& InitializationOptions)
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	MessageLogViewModel->RegisterLogListingViewModel(LogName, LogLabel, InitializationOptions);
}

bool FMessageLogModule::UnregisterLogListing(const FName& LogName)
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	return MessageLogViewModel->UnregisterLogListingViewModel(LogName);
}	

bool FMessageLogModule::IsRegisteredLogListing(const FName& LogName) const
{
	return MessageLogViewModel->IsRegisteredLogListingViewModel(LogName);
}

TSharedRef<IMessageLogListing> FMessageLogModule::CreateLogListing(const FName& InLogName, const FMessageLogInitializationOptions& InitializationOptions)
{
	LLM_SCOPE_BYTAG(EngineMisc_MessageLog);
	TSharedRef<FMessageLogListingModel> MessageLogListingModel = FMessageLogListingModel::Create( InLogName );
	return FMessageLogListingViewModel::Create( MessageLogListingModel, FText(), InitializationOptions );
}

TSharedRef<SWidget> FMessageLogModule::CreateLogListingWidget(const TSharedRef<IMessageLogListing>& InMessageLogListing)
{
	LLM_SCOPE(ELLMTag::UI);
	return SNew(SMessageLogListing, InMessageLogListing);
}

void FMessageLogModule::OpenMessageLog(const FName& LogName)
{
	if (GDisableMessageLogPopup)
	{
		return;
	}
	// only open the message log if we have a window created for the tab manager & our delegate allows it
	if(bCanDisplayMessageLog
#if !PLATFORM_MAC
	   && FGlobalTabmanager::Get()->GetRootWindow().IsValid()
#endif
	   )
	{
		FGlobalTabmanager::Get()->TryInvokeTab(FName("MessageLog"));
		MessageLogViewModel->ChangeCurrentListingViewModel(LogName);
	}
}

void FMessageLogModule::EnableMessageLogDisplay(bool bInCanDisplayMessageLog)
{
	bCanDisplayMessageLog = bInCanDisplayMessageLog;
}

void FMessageLogModule::RegisterMessageLogSpawner(const TSharedRef<FWorkspaceItem>& InGroup)
{
	LLM_SCOPE(ELLMTag::UI);
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner("MessageLog", FOnSpawnTab::CreateStatic(&SpawnMessageLog, MessageLogViewModel.ToSharedRef()))
		.SetDisplayName(NSLOCTEXT("UnrealEditor", "MessageLogTab", "Message Log"))
		.SetTooltipText(NSLOCTEXT("UnrealEditor", "MessageLogTooltipText", "Open the Message Log tab."))
		.SetGroup(InGroup)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "MessageLog.TabIcon"));
}
