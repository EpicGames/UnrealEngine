// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Logging/MessageLog.h" 

static void TestMessages()
{
	FMessageLog PIELogger = FMessageLog(FName("SlateStyleLog"));
	PIELogger.Message(EMessageSeverity::Type::Warning, FText::FromString("This is a Warning Message"));
	PIELogger.Message(EMessageSeverity::Type::Error, FText::FromString("This is an Error Message"));
	PIELogger.Message(EMessageSeverity::Type::Info, FText::FromString("This is an Info Message"));
}

FAutoConsoleCommand TestMessageLogCommand(TEXT("Slate.TestMessageLog"), TEXT(""), FConsoleCommandDelegate::CreateStatic(&TestMessages));