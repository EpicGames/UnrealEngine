// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/MessageLogListingModel.h"
#include "Logging/MessageLog.h"

#define LOCTEXT_NAMESPACE "Developer.MessageLog"


MessageContainer::TConstIterator FMessageLogListingModel::GetMessageIterator(const uint32 PageIndex) const
{
	return PageAtIndex(PageIndex)->Messages.CreateConstIterator();
}

const TSharedPtr< FTokenizedMessage >  FMessageLogListingModel::GetMessageAtIndex( const uint32 PageIndex, const int32 MessageIndex ) const
{
	TSharedPtr<FTokenizedMessage> FoundMessage = nullptr;
	FPage* Page = PageAtIndex(PageIndex);
	if( Page->Messages.IsValidIndex( MessageIndex ) )
	{
		FoundMessage = Page->Messages[MessageIndex];
	}
	return FoundMessage;
}

FString FMessageLogListingModel::GetAllMessagesAsString( const uint32 PageIndex ) const
{
	// Go through all the messages and add it to the compiled one.
	FPage* Page = PageAtIndex(PageIndex);
	TArray<FString> AllLines;
	for( int32 MessageID = 0; MessageID < Page->Messages.Num(); MessageID++ )
	{
		AllLines.Add(Page->Messages[ MessageID ]->ToText().ToString());
	}

	return FString::Join(AllLines, TEXT("\n"));
}

void FMessageLogListingModel::AddMessageInternal( const TSharedRef<FTokenizedMessage>& NewMessage, bool bMirrorToOutputLog, bool bDiscardDuplicates )
{
	if (bIsPrintingToOutputLog)
	{
		return;
	}

	if (bDiscardDuplicates)
	{
		bool bIsAlreadyInSet;
		CurrentPage().MessagesHashes.Add(GetTypeHash(NewMessage->ToText().ToString()), &bIsAlreadyInSet);
		if (bIsAlreadyInSet)
		{
			return;
		}
	}

	CurrentPage().Messages.Add( NewMessage );

	if (bMirrorToOutputLog)
	{
		// Prevent re-entrancy from the output log to message log mirroring code
		TGuardValue<bool> PrintToOutputLogBool(bIsPrintingToOutputLog, true);

		const TCHAR* const LogColor = FMessageLog::GetLogColor(NewMessage->GetSeverity());
		if (LogColor)
		{
			SET_WARN_COLOR(LogColor);
		}

		FMsg::Logf(__FILE__, __LINE__, UE_FNAME_TO_LOG_CATEGORY_NAME(LogName), FMessageLog::GetLogVerbosity(NewMessage->GetSeverity()), TEXT("%s"), *NewMessage->ToText().ToString());

		CLEAR_WARN_COLOR();
	}
}

void FMessageLogListingModel::AddMessage( const TSharedRef<FTokenizedMessage>& NewMessage, bool bMirrorToOutputLog, bool bDiscardDuplicates )
{
	CreateNewPageIfRequired();

	AddMessageInternal( NewMessage, bMirrorToOutputLog, bDiscardDuplicates );

	Notify();
}

void FMessageLogListingModel::AddMessages( const TArray< TSharedRef<class FTokenizedMessage> >& NewMessages, bool bMirrorToOutputLog, bool bDiscardDuplicates )
{
	CreateNewPageIfRequired();

	for( int32 MessageIdx = 0; MessageIdx < NewMessages.Num(); MessageIdx++ )
	{
		AddMessageInternal( NewMessages[MessageIdx], bMirrorToOutputLog, bDiscardDuplicates );
	}
	Notify();
}

void FMessageLogListingModel::ClearMessages()
{
	CurrentPage().Messages.Empty();
	CurrentPage().MessagesHashes.Empty();
	Notify();
}

void FMessageLogListingModel::NewPage( const FText& InTitle, uint32 InMaxPages )
{
	// store the name of the page we will add if a new message is pushed
	PendingPageName = InTitle;
	MaxPages = InMaxPages;

	// if the current output page has messages, we create a new page now
	if( CurrentPage().Messages.Num() > 0 )
	{
		CreateNewPageIfRequired();
	}
}

bool FMessageLogListingModel::SetCurrentPage(const FText& InTitle, uint32 InMaxPages )
{
	if (CurrentPage().Title.CompareTo(InTitle) == 0)
	{
		return false;
	}
	
	for (TDoubleLinkedList<FPage>::TIterator it = begin(Pages); it != end(Pages); ++it)
	{
		if (it.GetNode()->GetValue().Title.CompareTo(InTitle) == 0)
		{
			TDoubleLinkedList<FPage>::TDoubleLinkedListNode* SelectedNode = it.GetNode();
			Pages.RemoveNode(SelectedNode, false);
			Pages.AddHead(SelectedNode);

			// invalidate cache as all indices will change
			CachedPage = nullptr;
			return true;
		}
	}

	// If the page does not exist yet, create a new one
	NewPage(InTitle, InMaxPages);

	return true;
}

bool FMessageLogListingModel::SetCurrentPage( const uint32 InOldPageIndex  )
{
	if (InOldPageIndex == 0)
	{
		return false;
	}

	if (InOldPageIndex >= NumPages())
	{
		return false;
	}

	FPage* PageToSwitch = PageAtIndex(InOldPageIndex);
	return SetCurrentPage(PageToSwitch->Title, MaxPages);
}

uint32 FMessageLogListingModel::NumPages() const
{
	return Pages.Num();
}

uint32 FMessageLogListingModel::NumMessages( uint32 PageIndex ) const
{
	return PageAtIndex(PageIndex)->Messages.Num();
}

FMessageLogListingModel::FPage& FMessageLogListingModel::CurrentPage() const
{
	check(Pages.Num() > 0);
	return Pages.GetHead()->GetValue();
}

FMessageLogListingModel::FPage* FMessageLogListingModel::PageAtIndex(const uint32 PageIndex) const
{
	check(PageIndex < (uint32)Pages.Num());
	if(CachedPage != nullptr && CachedPageIndex == PageIndex)
	{
		return CachedPage;
	}
	else
	{
		uint32 CurrentIndex = 0;
		for(auto Node = Pages.GetHead(); Node; Node = Node->GetNextNode())
		{
			if(CurrentIndex == PageIndex)
			{
				CachedPage = &Node->GetValue();
				CachedPageIndex = CurrentIndex;
				return CachedPage;
			}
			CurrentIndex++;
		}
	}

	check(false);	// Should never get here!
	return nullptr;
}

const FText& FMessageLogListingModel::GetPageTitle( const uint32 PageIndex ) const
{
	return PageAtIndex(PageIndex)->Title;
}

void FMessageLogListingModel::CreateNewPageIfRequired()
{
	if(!PendingPageName.IsEmpty())
	{
		// dont create a new page if the current is empty, just change its name
		if(CurrentPage().Messages.Num() != 0)
		{
			// remove any pages that exceed the max page count
			while(Pages.Num() >= (int32)MaxPages)
			{
				Pages.RemoveNode(Pages.GetTail());
			}
			Pages.AddHead(FPage(PendingPageName));

			// invalidate cache as all indices will change
			CachedPage = nullptr;
		}
		else
		{
			CurrentPage().Title = PendingPageName;
		}

		PendingPageName = FText::GetEmpty();
	}
}

bool FMessageLogListingModel::AreMessagesEqual(const TSharedRef< FTokenizedMessage >& MessageA, const TSharedRef< FTokenizedMessage >& MessageB)
{
	if(MessageA->GetMessageTokens().Num() == MessageB->GetMessageTokens().Num())
	{
		auto TokenItA(MessageA->GetMessageTokens().CreateConstIterator());
		auto TokenItB(MessageB->GetMessageTokens().CreateConstIterator());
		for( ; TokenItA && TokenItB; ++TokenItA, ++TokenItB)
		{
			TSharedRef<IMessageToken> TokenA = *TokenItA;
			TSharedRef<IMessageToken> TokenB = *TokenItB;
			if ( TokenA->GetType() != TokenB->GetType() || !TokenA->ToText().EqualTo( TokenB->ToText() ) )
			{
				return false;
			}
		}

		return true;
	}

	return false;
}

// Specialized KeyFunc for TSharedRef<FTokenizedMessage> usage in TSet
struct FTokenizedMessageKeyFunc
{
	typedef typename TCallTraits<TSharedRef<FTokenizedMessage>>::ParamType KeyInitType;
	typedef typename TCallTraits<TSharedRef<FTokenizedMessage>>::ParamType ElementInitType;

	enum { bAllowDuplicateKeys = false };

	static FORCEINLINE KeyInitType GetSetKey(ElementInitType Element)
	{
		return Element;
	}

	static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B)
	{
		return FMessageLogListingModel::AreMessagesEqual(A, B);
	}

	static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key->ToText().ToString());
	}
};

void FMessageLogListingModel::RemoveDuplicates(uint32 PageIndex)
{
	FPage* Page = PageAtIndex(PageIndex);
	
	if (Page != nullptr && Page->Messages.Num() > 0)
	{
		TSet<TSharedRef<FTokenizedMessage>, FTokenizedMessageKeyFunc> MessageSet(Page->Messages);
		Page->Messages = MessageSet.Array();

		// Also needs to rebuild array of hashes
		Page->MessagesHashes.Empty(Page->Messages.Num());
		for(int32 MessageID = 0; MessageID < Page->Messages.Num(); MessageID++)
		{
			const TSharedPtr< FTokenizedMessage > Message = Page->Messages[MessageID];
			Page->MessagesHashes.Add(GetTypeHash(Message->ToText().ToString()));
		}
	}
}

#undef LOCTEXT_NAMESPACE
