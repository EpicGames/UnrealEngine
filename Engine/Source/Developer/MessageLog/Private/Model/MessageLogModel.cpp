// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/MessageLogModel.h"

FMessageLogModel::~FMessageLogModel()
{
	if (!NameToModelMap.IsEmpty())
	{
		NameToModelMap.Empty();
		Notify();
	}
}

TSharedRef<FMessageLogListingModel> FMessageLogModel::RegisterOrGetLogListingModel( const FName& LogName )
{
	check( LogName != NAME_None );

	TSharedPtr< FMessageLogListingModel > LogListingModel = FindLogListingModel( LogName );
	if( !LogListingModel.IsValid() )
	{
		LogListingModel = FMessageLogListingModel::Create( LogName );
		NameToModelMap.Add( LogName, LogListingModel );
	}

	return LogListingModel.ToSharedRef();
};

bool FMessageLogModel::UnregisterLogListingModel( const FName& LogName )
{
	check( LogName != NAME_None );

	int32 RemovedModel = NameToModelMap.Remove( LogName );
	if(RemovedModel > 0)
	{
		Notify();
		return true;
	}
	
	return false;
}

bool FMessageLogModel::IsRegisteredLogListingModel( const FName& LogName ) const
{
	check( LogName != NAME_None );
	return NameToModelMap.Find(LogName) != NULL;
}

TSharedPtr< FMessageLogListingModel > FMessageLogModel::FindLogListingModel( const FName& LogName ) const
{
	check( LogName != NAME_None );

	const TSharedPtr<FMessageLogListingModel>* Model = NameToModelMap.Find(LogName);
	if(Model != NULL)
	{
		return *Model;
	}
	return NULL;
}

TSharedRef< FMessageLogListingModel > FMessageLogModel::GetLogListingModel( const FName& LogName )
{
	check( LogName != NAME_None );
	return RegisterOrGetLogListingModel( LogName );
}
