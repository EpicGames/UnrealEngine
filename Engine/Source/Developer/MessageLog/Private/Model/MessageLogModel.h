// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Model/MessageLogListingModel.h"

/**
 * Holds a series of message log listings that can be written to.
 */
class FMessageLogModel
{
public:

	/** Destructor */ 
	virtual ~FMessageLogModel();

	/** Unregisters a log listing with the message log system, true if successful */
	bool UnregisterLogListingModel( const FName& LogName );

	/** Checks to see if a log listing is already registered with the system */
	bool IsRegisteredLogListingModel( const FName& LogName ) const;

	/** Gets a log listing, if it does not exist it is created. */
	TSharedRef< class FMessageLogListingModel > GetLogListingModel( const FName& LogName );

	/** Broadcasts whenever a message log listing is added or removed */
	DECLARE_EVENT( FMessageLogModel, FChangedEvent )
	FChangedEvent& OnChanged() { return ChangedEvent; }

protected:
	/** Will broadcast to all registered observers informing them of a change */
	virtual void Notify() { ChangedEvent.Broadcast(); }

private:

	/** Registers a log listing with the message log system, true if successful */
	TSharedRef< class FMessageLogListingModel > RegisterOrGetLogListingModel( const FName& LogName );

	/** Finds the LogListing Model, given its name.  Returns null if not found. */
	TSharedPtr< class FMessageLogListingModel > FindLogListingModel( const FName& LogName ) const;

private:

	/** A map from a log listings' Name->Model */
	TMap< FName, TSharedPtr< class FMessageLogListingModel > > NameToModelMap;

	/**	The event that broadcasts whenever data is changed */
	FChangedEvent ChangedEvent;
};
