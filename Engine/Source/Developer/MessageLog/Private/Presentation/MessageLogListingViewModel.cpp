// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presentation/MessageLogListingViewModel.h"
#include "Modules/ModuleManager.h"
#include "MessageFilter.h"
#include "MessageLogModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "MessageLog"

int32 FMessageLogListingViewModel::NextNotificationId = 0;

FMessageLogListingViewModel::FMessageLogListingViewModel() = default;

FMessageLogListingViewModel::~FMessageLogListingViewModel()
{
	check(MessageLogListingModel.IsValid());
	MessageLogListingModel->OnChanged().RemoveAll(this);

	for( auto It = MessageFilters.CreateConstIterator(); It; ++It)
	{
		const TSharedPtr<FMessageFilter> Filter = *It;
		if(Filter.IsValid())
		{
			Filter->OnFilterChanged().RemoveAll(this);
		}
	}
}

void FMessageLogListingViewModel::Initialize()
{
	check(MessageLogListingModel.IsValid());

	// Register with the model so that if it changes we get updates
	MessageLogListingModel->OnChanged().AddSP(this, &FMessageLogListingViewModel::OnChanged);

	// Create our filters
	MessageFilters.Add(MakeShareable(new FMessageFilter(LOCTEXT("CriticalErrors", "Critical Errors"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ErrorWithColor"))));
	MessageFilters.Add(MakeShareable(new FMessageFilter(LOCTEXT("Errors", "Errors"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ErrorWithColor"))));
	MessageFilters.Add(MakeShareable(new FMessageFilter(LOCTEXT("PerformanceWarnings", "Performance Warnings"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.WarningWithColor"))));
	MessageFilters.Add(MakeShareable(new FMessageFilter(LOCTEXT("Warnings", "Warnings"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.WarningWithColor"))));
	MessageFilters.Add(MakeShareable(new FMessageFilter(LOCTEXT("Info", "Info"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.BulletPoint"))));

	for( auto It = MessageFilters.CreateConstIterator(); It; ++It)
	{
		const TSharedPtr<FMessageFilter> Filter = *It;
		if(Filter.IsValid())
		{
			Filter->OnFilterChanged().AddRaw(this, &FMessageLogListingViewModel::OnFilterChanged);
		}
	}
}

void FMessageLogListingViewModel::OnFilterChanged()
{
	RefreshFilteredMessages();
}

void FMessageLogListingViewModel::OnChanged()
{
	check( !bIsRefreshing );
	bIsRefreshing = true;

	RefreshFilteredMessages();

	bIsRefreshing = false;
}

MessageContainer::TConstIterator FMessageLogListingViewModel::GetFilteredMessageIterator() const
{
	return FilteredMessages.CreateConstIterator();
}

MessageContainer::TConstIterator FMessageLogListingViewModel::GetSelectedMessageIterator() const
{
	return SelectedFilteredMessages.CreateConstIterator();
}

const TSharedPtr<FTokenizedMessage>  FMessageLogListingViewModel::GetMessageAtIndex(const int32 MessageIndex) const
{
	TSharedPtr<FTokenizedMessage> FoundMessage = NULL;
	if(FilteredMessages.IsValidIndex(MessageIndex))
	{
		FoundMessage = FilteredMessages[MessageIndex];
	}
	return FoundMessage;
}

const TArray< TSharedRef<FMessageFilter> >& FMessageLogListingViewModel::GetMessageFilters() const
{
	return MessageFilters;
}

void FMessageLogListingViewModel::DismissNotification(int32 NotificationId)
{
	int32 NotificationIndex = OpenNotifications.IndexOfByPredicate([NotificationId](const FOpenNotification& N) { return N.NotificationId == NotificationId; });
	if (NotificationIndex != INDEX_NONE)
	{
		TSharedPtr<SNotificationItem> NotificationItem = OpenNotifications[NotificationIndex].NotificationItem.Pin();
		if (NotificationItem.IsValid())
		{
			NotificationItem->SetExpireDuration(0.0f);
			NotificationItem->ExpireAndFadeout();
		}

		OpenNotifications.RemoveAtSwap(NotificationIndex);
	}
}

void FMessageLogListingViewModel::OpenMessageLogFromNotification(int32 NotificationId)
{
	DismissNotification(NotificationId);
	OpenMessageLog();
}

void FMessageLogListingViewModel::OpenMessageLog( )
{
	FMessageLogModule& MessageLogModule = FModuleManager::GetModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.OpenMessageLog(GetName());
}

int32 FMessageLogListingViewModel::NumMessagesPresent( uint32 PageIndex, EMessageSeverity::Type InSeverity ) const
{
	int32 Count = 0;
	for(MessageContainer::TConstIterator It(MessageLogListingModel->GetMessageIterator(PageIndex)); It; ++It)
	{
		const TSharedRef<FTokenizedMessage> Message = *It;
		if(Message->GetSeverity() <= InSeverity)
		{
			Count++;
		}
	}

	return Count;
}

EMessageSeverity::Type FMessageLogListingViewModel::HighestSeverityPresent( uint32 PageIndex ) const
{
	EMessageSeverity::Type Severity = EMessageSeverity::Info;
	for(MessageContainer::TConstIterator It(MessageLogListingModel->GetMessageIterator(PageIndex)); It; ++It)
	{
		const TSharedRef<FTokenizedMessage> Message = *It;
		if(Message->GetSeverity() < Severity)
		{
			Severity = Message->GetSeverity();
		}
	}

	return Severity;
}

void FMessageLogListingViewModel::AddMessage( const TSharedRef< FTokenizedMessage >& NewMessage, bool bMirrorToOutputLog )
{
	MessageLogListingModel->AddMessage(NewMessage, bMirrorToOutputLog, bDiscardDuplicates);
}

void FMessageLogListingViewModel::AddMessages( const TArray< TSharedRef< FTokenizedMessage > >& NewMessages, bool bMirrorToOutputLog )
{
	MessageLogListingModel->AddMessages(NewMessages, bMirrorToOutputLog, bDiscardDuplicates);
}

void FMessageLogListingViewModel::ClearMessages()
{
	if (CurrentPageIndex != 0)
	{
		SetCurrentPage(CurrentPageIndex);
	}

	MessageLogListingModel->ClearMessages();
}

void FMessageLogListingViewModel::SelectMessages( const TArray< TSharedRef<FTokenizedMessage> >& InSelectedMessages )
{
	SelectedFilteredMessages.Empty();
	SelectedFilteredMessages.Append(InSelectedMessages);
	SelectionChangedEvent.Broadcast();
}

const TArray< TSharedRef<FTokenizedMessage> >& FMessageLogListingViewModel::GetSelectedMessages() const
{
	return SelectedFilteredMessages;
}

const TArray< TSharedRef<FTokenizedMessage> >& FMessageLogListingViewModel::GetFilteredMessages() const
{
	return FilteredMessages;
}

void FMessageLogListingViewModel::SelectMessage(const TSharedRef<FTokenizedMessage>& Message, bool bSelected)
{
	bool bIsAlreadySelected = INDEX_NONE != SelectedFilteredMessages.Find( Message );

	if(bSelected && !bIsAlreadySelected)
	{
		bool bIsInFilteredList = INDEX_NONE != FilteredMessages.Find(Message);
		if(bIsInFilteredList)
		{
			SelectedFilteredMessages.Add( Message );
			SelectionChangedEvent.Broadcast();
		}
	}
	else if(!bSelected && bIsAlreadySelected)
	{
		SelectedFilteredMessages.Remove( Message );
		SelectionChangedEvent.Broadcast();
	}
}

bool FMessageLogListingViewModel::IsMessageSelected(const TSharedRef<FTokenizedMessage>& Message) const
{
	return INDEX_NONE != SelectedFilteredMessages.Find(Message);
}

void FMessageLogListingViewModel::ClearSelectedMessages()
{
	SelectedFilteredMessages.Empty();
	SelectionChangedEvent.Broadcast();
}

void FMessageLogListingViewModel::InvertSelectedMessages()
{
	for(MessageContainer::TConstIterator It(GetFilteredMessageIterator()); It; ++It)
	{
		const TSharedRef<FTokenizedMessage> Message = *It;
		const bool bSelected = IsMessageSelected(Message);
		SelectMessage(Message, !bSelected);
	}
	SelectionChangedEvent.Broadcast();
}

FString FMessageLogListingViewModel::GetSelectedMessagesAsString() const
{
	// Go through each selected message and add it to the compiled one.
	TArray< TSharedRef< FTokenizedMessage > > Selection = GetSelectedMessages();
	TArray<FString> SelectedLines;
	for( int32 MessageID = 0; MessageID < Selection.Num(); MessageID++ )
	{
		SelectedLines.Add(Selection[MessageID]->ToText().ToString());
	}

	return FString::Join(SelectedLines, TEXT("\n"));
}

FString FMessageLogListingViewModel::GetAllMessagesAsString() const
{
	return MessageLogListingModel->GetAllMessagesAsString(CurrentPageIndex);
}

const FName& FMessageLogListingViewModel::GetName() const
{
	return MessageLogListingModel->GetName(); 
}

void FMessageLogListingViewModel::SetLabel( const FText& InLogLabel )
{
	LogLabel = InLogLabel;
}

const FText& FMessageLogListingViewModel::GetLabel() const 
{ 
	return LogLabel;
}

void FMessageLogListingViewModel::ExecuteToken( const TSharedRef<IMessageToken>& Token ) const
{
	TokenClickedEvent.Broadcast( Token );
}

void FMessageLogListingViewModel::NewPage( const FText& Title )
{
	// we should take this as a suggestion we want to show pages!
	bShowPages = true;

	// reset so we always display the new page when we switch
	CurrentPageIndex = 0;

	// add new page & refresh
	MessageLogListingModel->NewPage( Title, MaxPageCount );
	RefreshFilteredMessages();
}

void FMessageLogListingViewModel::SetCurrentPage(const FText& Title)
{
	// we should take this as a suggestion we want to show pages!
	bShowPages = true;

	// switch page & refresh
	MessageLogListingModel->SetCurrentPage( Title, MaxPageCount );
	
	// reset so we always display the current page when we switch
	SetCurrentPageIndex(0);
}

void FMessageLogListingViewModel::SetCurrentPage(const uint32 InOldPageIndex)
{
	// we should take this as a suggestion we want to show pages!
	bShowPages = true;

	// switch page & refresh
	MessageLogListingModel->SetCurrentPage( InOldPageIndex );
	
	// reset so we always display the current page when we switch
	SetCurrentPageIndex(0);
}

void FMessageLogListingViewModel::NotifyIfAnyMessages( const FText& Message, EMessageSeverity::Type SeverityFilter, bool bForce )
{
	// Note we use page 0 in this function, as that is the page that will
	// have most recently had messages added to it.

	// SeverityFilter represents only logging items Higher severity than it.
	EMessageSeverity::Type HigherSeverity = ( EMessageSeverity::Type )( FMath::Max(SeverityFilter - 1, 0) );

	if ( bForce || NumMessagesPresent(0, HigherSeverity) > 0 )
	{
		FText NotificationMessage;
		if(Message.IsEmpty())
		{
			if(MessageLogListingModel->NumMessages(0) > 0)
			{
				// no message passed in, so we need to make a default - use the last message we output
				FFormatNamedArguments Args;
				Args.Add( TEXT("LogLabel"), LogLabel );
				Args.Add( TEXT("LastMessage"), MessageLogListingModel->GetMessageAtIndex(0, MessageLogListingModel->NumMessages(0) - 1)->ToText() );
				NotificationMessage = FText::Format( LOCTEXT("DefaultNoMessageToLastMessage", "{LogLabel}: {LastMessage}"), Args );
			}
			else
			{
				// no present messages & no message passed in, use the log label as a default
				NotificationMessage = LogLabel;
			}
		}
		else
		{
			NotificationMessage = Message;
		}

		FNotificationInfo ErrorNotification(NotificationMessage);

		if (NumMessagesPresent(0, EMessageSeverity::Error) > 0)
		{
			// If there is a notification with the same message, dismiss it to make space for this new one.
			const FOpenNotification* ExistingNotification = OpenNotifications.FindByPredicate([NotificationMessage](const FOpenNotification& Notification)
			{
				return Notification.NotificationMessage.CompareTo(NotificationMessage) == 0;
			});
			if(ExistingNotification != nullptr)
			{
				DismissNotification(ExistingNotification->NotificationId);
			}
			
			int32 NotificationId = NextNotificationId++;

			ErrorNotification.bFireAndForget = true;
			ErrorNotification.bUseThrobber = false;
			ErrorNotification.FadeOutDuration = 0.f;
			ErrorNotification.ExpireDuration = 15.f;
			ErrorNotification.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("DismissMessageButton", "Dismiss"), FText(), FSimpleDelegate::CreateSP(this, &FMessageLogListingViewModel::DismissNotification, NotificationId), SNotificationItem::CS_Fail));
			ErrorNotification.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("ShowMessageLogButton", "Show Message Log"), FText(), FSimpleDelegate::CreateSP(this, &FMessageLogListingViewModel::OpenMessageLogFromNotification, NotificationId), SNotificationItem::CS_Fail));

			TSharedPtr<SNotificationItem> NewNotificationItem = FSlateNotificationManager::Get().AddNotification(ErrorNotification);
			if (NewNotificationItem.IsValid())
			{
				NewNotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
				OpenNotifications.Emplace(NotificationId, NewNotificationItem, NotificationMessage);
			}
		}
		else
		{
			ErrorNotification.bFireAndForget = true;
			ErrorNotification.Hyperlink = FSimpleDelegate::CreateSP(this, &FMessageLogListingViewModel::OpenMessageLog);
			ErrorNotification.HyperlinkText = LOCTEXT("ShowMessageLogHyperlink", "Show Message Log");
			ErrorNotification.ExpireDuration = 14.0f; // Need this message to last a little longer than normal since the user may want to "Show Log"
			ErrorNotification.bUseThrobber = true;

			FSlateNotificationManager::Get().AddNotification(ErrorNotification);
		}
	}
}

int32 FMessageLogListingViewModel::NumMessages( EMessageSeverity::Type SeverityFilter )
{
	return NumMessagesPresent(0, SeverityFilter);
}

void FMessageLogListingViewModel::Open()
{
	OpenMessageLog();
}

void FMessageLogListingViewModel::RefreshFilteredMessages()
{
	FilteredMessages.Empty();
	for(MessageContainer::TConstIterator It(MessageLogListingModel->GetMessageIterator(CurrentPageIndex)); It; ++It)
	{
		const TSharedRef<FTokenizedMessage> Message = *It;

		if(MessageFilters.Num() > 0)
		{
			if( !MessageFilters.IsValidIndex(Message->GetSeverity()) || MessageFilters[Message->GetSeverity()]->GetDisplay() )
			{
				FilteredMessages.Add(Message);
			}
		}
		else
		{
			FilteredMessages.Add(Message);
		}
	}

	// Re-broadcasts to anything that is registered
	ChangedEvent.Broadcast();
}

void FMessageLogListingViewModel::SetShowFilters(bool bInShowFilters)
{
	bShowFilters = bInShowFilters;
}

bool FMessageLogListingViewModel::GetShowFilters() const
{
	return bShowFilters;
}

void FMessageLogListingViewModel::SetShowPages(bool bInShowPages)
{
	bShowPages = bInShowPages;
}

bool FMessageLogListingViewModel::GetShowPages() const
{
	return bShowPages;
}

void FMessageLogListingViewModel::SetScrollToBottom(bool bInScrollToBottom)
{
	bScrollToBottom = bInScrollToBottom;
}

bool FMessageLogListingViewModel::GetScrollToBottom() const
{
	return bScrollToBottom;
}

void FMessageLogListingViewModel::SetAllowClear(bool bInAllowClear)
{
	bAllowClear = bInAllowClear;
}

bool FMessageLogListingViewModel::GetAllowClear() const
{
	return bAllowClear;
}

void FMessageLogListingViewModel::SetDiscardDuplicates(bool bInDiscardDuplicates)
{
	bool bOldDiscardDuplicates = bDiscardDuplicates;
	bDiscardDuplicates = bInDiscardDuplicates;
	if(bDiscardDuplicates && bOldDiscardDuplicates != bDiscardDuplicates)
	{
		// remove any duplicate messages currently in the log, as we might have added duplicate messages
		// before this listing was registered
		MessageLogListingModel->RemoveDuplicates(0);
		RefreshFilteredMessages();
	}
}

bool FMessageLogListingViewModel::GetDiscardDuplicates() const
{
	return bDiscardDuplicates;
}

void FMessageLogListingViewModel::SetMaxPageCount(uint32 InMaxPageCount)
{
	MaxPageCount = InMaxPageCount;
}

uint32 FMessageLogListingViewModel::GetMaxPageCount() const
{
	return MaxPageCount;
}
	
uint32 FMessageLogListingViewModel::GetPageCount() const
{
	return MessageLogListingModel->NumPages();
}

uint32 FMessageLogListingViewModel::GetCurrentPageIndex() const
{
	return CurrentPageIndex;
}

void FMessageLogListingViewModel::SetCurrentPageIndex( uint32 InCurrentPageIndex )
{
	CurrentPageIndex = InCurrentPageIndex;
	PageSelectionChangedEvent.Broadcast();
	RefreshFilteredMessages();
}

void FMessageLogListingViewModel::NextPage()
{
	CurrentPageIndex = CurrentPageIndex == 0 ? MessageLogListingModel->NumPages() - 1 : CurrentPageIndex - 1;
	PageSelectionChangedEvent.Broadcast();
	RefreshFilteredMessages();
}

void FMessageLogListingViewModel::PrevPage()
{
	CurrentPageIndex = CurrentPageIndex == MessageLogListingModel->NumPages() - 1 ? 0 : CurrentPageIndex + 1;
	PageSelectionChangedEvent.Broadcast();
	RefreshFilteredMessages();
}

const FText& FMessageLogListingViewModel::GetPageTitle( const uint32 PageIndex ) const
{
	return MessageLogListingModel->GetPageTitle(PageIndex);
}

uint32 FMessageLogListingViewModel::NumMessages() const
{
	return MessageLogListingModel->NumMessages(CurrentPageIndex);
}

#undef LOCTEXT_NAMESPACE
