// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presentation/MessageLogViewModel.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"

#define LOCTEXT_NAMESPACE "MessageLog"

DEFINE_LOG_CATEGORY_STATIC(LogMessageLog, Log, All);

FMessageLogViewModel::FMessageLogViewModel( const TSharedPtr< FMessageLogModel >& InMessageLogModel )
	: MessageLogModel( InMessageLogModel )
	, SelectedLogListingViewModel( NULL )
{
}

FMessageLogViewModel::~FMessageLogViewModel()
{
	check(MessageLogModel.IsValid());
	MessageLogModel->OnChanged().RemoveAll( this );
}

void FMessageLogViewModel::Initialize()
{
	check( MessageLogModel.IsValid() );

	// Register with the model so that if it changes we get updates
	MessageLogModel->OnChanged().AddSP( this, &FMessageLogViewModel::Update );
}

void FMessageLogViewModel::Update()
{
	// Re-broadcasts to anything that is registered
	ChangedEvent.Broadcast();
}

void FMessageLogViewModel::ChangeCurrentListingViewModel( const FName& LogName )
{
	TSharedPtr<FMessageLogListingViewModel> LogListingViewModel = FindLogListingViewModel( LogName );
	if( LogListingViewModel.IsValid() )
	{
		if ( FPaths::FileExists(GEditorPerProjectIni) )
		{
			GConfig->SetString( TEXT("MessageLog"), TEXT("LastLogListing"), *LogName.ToString(), GEditorPerProjectIni );
		}
		SelectedLogListingViewModel = LogListingViewModel;
		SelectionChangedEvent.Broadcast();		
	}
}

TSharedPtr<FMessageLogListingViewModel> FMessageLogViewModel::GetCurrentListingViewModel() const
{
	return SelectedLogListingViewModel;
}

FName FMessageLogViewModel::GetCurrentListingName() const
{
	if( SelectedLogListingViewModel.IsValid() )
	{
		return SelectedLogListingViewModel->GetName();
	}
	return FName();
}

FString FMessageLogViewModel::GetCurrentListingLabel() const
{
	if( SelectedLogListingViewModel.IsValid() )
	{
		return SelectedLogListingViewModel->GetLabel().ToString();
	}
	return FString();
}

TSharedRef<FMessageLogListingViewModel> FMessageLogViewModel::RegisterLogListingViewModel( const FName& LogName, const FText& LogLabel, const FMessageLogInitializationOptions& InitializationOptions )
{
	check( LogName != NAME_None );

	TSharedPtr< FMessageLogListingViewModel > LogListingViewModel = FindLogListingViewModel( LogName );
	if( !LogListingViewModel.IsValid() )
	{
		LogListingViewModel = FMessageLogListingViewModel::Create( MessageLogModel->GetLogListingModel(LogName), LogLabel, InitializationOptions );
		NameToViewModelMap.Add( LogName, LogListingViewModel );
		UpdateListingViewModelArray();
	}
	else
	{
		// ViewModel may have been created earlier, as we can use it before any UI is constructed.
		// Therefore we may need to set its label here (and its show filters flag, pages, etc.)
		LogListingViewModel->SetLabel(LogLabel);
		LogListingViewModel->SetShowFilters(InitializationOptions.bShowFilters);
		LogListingViewModel->SetShowPages(InitializationOptions.bShowPages);
		LogListingViewModel->SetDiscardDuplicates(InitializationOptions.bDiscardDuplicates);
		LogListingViewModel->SetMaxPageCount(InitializationOptions.MaxPageCount);
	}

	return LogListingViewModel.ToSharedRef();
}

bool FMessageLogViewModel::UnregisterLogListingViewModel( const FName& LogName )
{
	check( LogName != NAME_None );
	return NameToViewModelMap.Remove( LogName ) > 0;
}

bool FMessageLogViewModel::IsRegisteredLogListingViewModel( const FName& LogName ) const
{
	check( LogName != NAME_None );
	return NameToViewModelMap.Find(LogName) != NULL;
}

TSharedPtr< FMessageLogListingViewModel > FMessageLogViewModel::FindLogListingViewModel( const FName& LogName ) const
{
	check( LogName != NAME_None );

	const TSharedPtr<FMessageLogListingViewModel>* ViewModel = NameToViewModelMap.Find(LogName);
	if(ViewModel != NULL)
	{
		return *ViewModel;
	}
	return NULL;
}

TSharedRef< FMessageLogListingViewModel > FMessageLogViewModel::GetLogListingViewModel( const FName& LogName )
{
	check( LogName != NAME_None );
	TSharedPtr< FMessageLogListingViewModel > LogListingViewModel = FindLogListingViewModel( LogName );
	if( !LogListingViewModel.IsValid() )
	{
		LogListingViewModel = FMessageLogListingViewModel::Create( MessageLogModel->GetLogListingModel(LogName), FText() );
		NameToViewModelMap.Add( LogName, LogListingViewModel );
		UpdateListingViewModelArray();
	}

	return LogListingViewModel.ToSharedRef();
}

void FMessageLogViewModel::UpdateListingViewModelArray()
{
	ViewModelArray.Empty();

	for(auto It = NameToViewModelMap.CreateConstIterator(); It; ++It)
	{
		if (It.Value()->ShouldShowInLogWindow())
		{
			ViewModelArray.Add(It.Value());
		}
	}

	ViewModelArray.Sort([](const IMessageLogListingPtr& lhs, const IMessageLogListingPtr& rhs) {
		return lhs->GetLabel().CompareTo(rhs->GetLabel()) <= 0;
	});

	Update();
}

#undef LOCTEXT_NAMESPACE
