// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Presentation/MessageLogListingViewModel.h"
#include "Model/MessageLogModel.h"

/** Presentation logic for the message log */
class FMessageLogViewModel : public TSharedFromThis< FMessageLogViewModel >
{

public:

	/** Broadcasts whenever we are informed of a change in the MessageLogModel */
	DECLARE_EVENT( FMessageLogViewModel, FChangedEvent )
	FChangedEvent& OnChanged() { return ChangedEvent; }

	/** Broadcasts whenever selection state changes */
	DECLARE_EVENT( FMessageLogViewModel, FOnSelectionChangedEvent )
	FOnSelectionChangedEvent& OnSelectionChanged() { return SelectionChangedEvent; }

public:

	/** Constructor */
	FMessageLogViewModel( const TSharedPtr< class FMessageLogModel >& InMessageLogModel );

	/** Destructor */ 
	virtual ~FMessageLogViewModel();

	/** Initializes the FMessageLogViewModel for use */
	virtual void Initialize();

	/** Called when data is changed changed/updated in the model */
	virtual void Update();

	/** 
	 * Registers a log listing view model.
	 * It is not necessary to call this function before outputting to a log via AddMessage etc. This call simply
	 * registers a UI to view the log data.
	 *
	 * @param	LogName					The name of the log to register
	 * @param	LogLabel				The label to display for the log
	 * @param	InitializationOptions	Initialization options for this message log
	 */
	TSharedRef<class FMessageLogListingViewModel> RegisterLogListingViewModel( const FName& LogName, const FText& LogLabel, const struct FMessageLogInitializationOptions& InitializationOptions );

	/** 
	 * Unregisters a log listing view model.
	 *
	 * @param	LogName		The name of the log to unregister.
	 * @returns true if successful.
	 */
	bool UnregisterLogListingViewModel( const FName& LogName );

	/** 
	 * Checks to see if a log listing view model is already registered
	 *
	 * @param	LogName		The name of the log to check.
	 * @returns true if the log listing is already registered.
	 */
	bool IsRegisteredLogListingViewModel( const FName& LogName ) const;

	/** Finds the LogListing ViewModel, given its name. Returns null if not found. */
	TSharedPtr< class FMessageLogListingViewModel > FindLogListingViewModel( const FName& LogName ) const;

	/** Gets a log listing ViewModel, if it does not exist it is created. */
	TSharedRef< class FMessageLogListingViewModel > GetLogListingViewModel( const FName& LogName );

	/** Changes the currently selected log listing */
	void ChangeCurrentListingViewModel( const FName& LogName );

	/** Gets the currently selected log listing */
	TSharedPtr<class FMessageLogListingViewModel> GetCurrentListingViewModel() const;

	/** Gets the currently selected log listing's name */
	FName GetCurrentListingName() const;

	/** Gets the currently selected log listing's label */
	FString GetCurrentListingLabel() const;

	/** Get the linearized array of ViewModels */
	const TArray<IMessageLogListingPtr>& GetLogListingViewModels() const { return ViewModelArray; }

private:

	/** Updates the linearized array of ViewModels */
	void UpdateListingViewModelArray();

private:

	/* The model we are getting display info from */
	TSharedPtr< class FMessageLogModel > MessageLogModel;

	/** A map from a log listings' Name->ViewModel */
	TMap< FName, TSharedPtr< class FMessageLogListingViewModel > > NameToViewModelMap;

	/** A linearized array of the ViewModels - we need this for the data to display in a SComboBox */
	TArray<IMessageLogListingPtr> ViewModelArray;

	/** The currently selected log listing */
	TSharedPtr< class FMessageLogListingViewModel > SelectedLogListingViewModel;

	/**	The event that broadcasts whenever a change occurs to the data */
	FChangedEvent ChangedEvent;

	/**	The event that broadcasts whenever selection state is changed */
	FOnSelectionChangedEvent SelectionChangedEvent;

};
