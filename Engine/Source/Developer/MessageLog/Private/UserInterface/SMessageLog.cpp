// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/SMessageLog.h"
#include "Widgets/Layout/SSplitter.h"
#include "Misc/ConfigCacheIni.h"
#include "SlateOptMacros.h"
#include "Presentation/MessageLogViewModel.h"
#include "UserInterface/SMessageLogListing.h"
#include "Widgets/Views/SListView.h"
#include "UserInterface/SMessageLogCategoryListRow.h"


#define LOCTEXT_NAMESPACE "SMessageLog"


/* SMessageLog structors
 *****************************************************************************/

SMessageLog::~SMessageLog()
{
	ViewModel->OnSelectionChanged().RemoveAll(this);
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMessageLog::Construct( const FArguments& InArgs, const TSharedRef<FMessageLogViewModel>& InViewModel )
{
	ViewModel = InViewModel;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(FMargin(20.0f, 13.97f, 16.f, 16.f))
		[
			SNew(SSplitter)
				.Orientation(Orient_Horizontal)
				.Style(FAppStyle::Get(), "SplitterPanel")
				.PhysicalSplitterHandleSize(10.0f)

			+ SSplitter::Slot()
				.Value(0.28f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
						.Padding(FMargin(16.f, 8.f, 0, 0))
						[
							// log categories list
							SAssignNew(CategoriesListView, SListView<IMessageLogListingPtr>)
								.ListItemsSource(&ViewModel->GetLogListingViewModels())
								.OnGenerateRow(this, &SMessageLog::HandleCategoriesListGenerateRow)
								.OnSelectionChanged(this, &SMessageLog::HandleCategoriesListSelectionChanged)
								.SelectionMode(ESelectionMode::Single)
								.HeaderRow
								(
									SNew(SHeaderRow)
										.Visibility(EVisibility::Collapsed)

									+ SHeaderRow::Column("Name")
										.DefaultLabel(LOCTEXT("CategoriesListNameColumnHeader", "Category"))
										.FillWidth(1.0)
								)
						]
				]

			+ SSplitter::Slot()
				.Value(0.72f)
				[
					SAssignNew(CurrentListingDisplay, SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
						.Padding(0.0f)
				]
		]

	];

	// Default to the first available (or the last used) listing
	const TArray<IMessageLogListingPtr>& ViewModels = ViewModel->GetLogListingViewModels();

	if (ViewModels.Num() > 0)
	{
		IMessageLogListingPtr DefaultViewModel = ViewModels[0];

		if (FPaths::FileExists(GEditorPerProjectIni))
		{
			FString LogName;

			if (GConfig->GetString(TEXT("MessageLog"), TEXT("LastLogListing"), LogName, GEditorPerProjectIni))
			{
				IMessageLogListingPtr LoadedViewModel = ViewModel->FindLogListingViewModel(FName(*LogName));
				
				if (LoadedViewModel.IsValid())
				{
					DefaultViewModel = LoadedViewModel;
				}
			}
		}

		CategoriesListView->SetSelection(DefaultViewModel, ESelectInfo::Direct);
	}

	ViewModel->OnSelectionChanged().AddSP(this, &SMessageLog::HandleSelectionUpdated);
	ViewModel->OnChanged().AddSP(this, &SMessageLog::RefreshCategoryList);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SMessageLog::HandleSelectionUpdated()
{
	TSharedPtr<FMessageLogListingViewModel> LogListingViewModel = ViewModel->GetCurrentListingViewModel();

	if ( LogListingViewModel.IsValid() )
	{
		if ( !CategoriesListView->IsItemSelected(LogListingViewModel) )
		{
			CategoriesListView->SetSelection(LogListingViewModel, ESelectInfo::Direct);
		}
	}
}

void SMessageLog::RefreshCategoryList()
{
	CategoriesListView->RequestListRefresh();
}

/* SWidget overrides
 *****************************************************************************/

FReply SMessageLog::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if (LogListing.IsValid())
	{
		return LogListing->GetCommandList()->ProcessCommandBindings(InKeyEvent)
			? FReply::Handled()
			: FReply::Unhandled();
	}
	
	return FReply::Unhandled();
}


/* SMessageLog callbacks
 *****************************************************************************/

TSharedRef<ITableRow> SMessageLog::HandleCategoriesListGenerateRow( IMessageLogListingPtr Item, const TSharedRef<STableViewBase>& OwnerTable )
{
	return SNew(SMessageLogCategoryListRow, Item.ToSharedRef(), OwnerTable);
}

	
void SMessageLog::HandleCategoriesListSelectionChanged( IMessageLogListingPtr Selection, ESelectInfo::Type SelectInfo )
{
	if (Selection.IsValid())
	{
		CurrentListingDisplay->SetContent(SAssignNew(LogListing, SMessageLogListing, Selection.ToSharedRef()));
	}
	else
	{
		CurrentListingDisplay->SetContent(SNullWidget::NullWidget);
	}
}


#undef LOCTEXT_NAMESPACE
