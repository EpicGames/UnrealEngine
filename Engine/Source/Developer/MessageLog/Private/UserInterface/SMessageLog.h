// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageLogListing.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

class FMessageLogViewModel;
class SMessageLogListing;

/**
 * An implementation for the message log widget.
 *
 * It holds a series of message log listings which it can switch between.
 */
class SMessageLog
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessageLog){}
	SLATE_END_ARGS()

	static FName AppName;
	
	/** Destructor. */
	~SMessageLog();

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InModel The view model to use.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<class FMessageLogViewModel>& InViewModel );

public:

	// SWidget overrides

	FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

private:

	/** Callback for generating a row in the categories list. */
	TSharedRef<ITableRow> HandleCategoriesListGenerateRow( IMessageLogListingPtr Item, const TSharedRef<STableViewBase>& OwnerTable );
	
	/** Callback for changing the selected item in the categories list. */
	void HandleCategoriesListSelectionChanged( IMessageLogListingPtr Selection, ESelectInfo::Type SelectInfo );

	/** Handles the broadcast message sent when the current log listing selection is changed */
	void HandleSelectionUpdated();

	/** Handles the broadcast message sent when the current log listing is updated */
	void RefreshCategoryList();

private:	

	/** Holds the log categories list view widget. */
	TSharedPtr<SListView<IMessageLogListingPtr>> CategoriesListView;

	/** The widget for displaying the current listing. */
	TSharedPtr<SBorder> CurrentListingDisplay;

	/** The current log listing widget, if any. */
	TSharedPtr<class SMessageLogListing> LogListing;

	/** The message log view model */
	TSharedPtr<class FMessageLogViewModel> ViewModel;
};
