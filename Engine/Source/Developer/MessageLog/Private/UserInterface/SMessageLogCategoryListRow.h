// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageLogListing.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "SlateOptMacros.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SMessageLogCategoryListRow"

/**
 * Implements a row widget for the log categories list view.
 */
class SMessageLogCategoryListRow
	: public SMultiColumnTableRow<IMessageLogListingPtr>
{
public:

	SLATE_BEGIN_ARGS(SMessageLogCategoryListRow) { }
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The arguments.
	 */
	void Construct( const FArguments& InArgs, const IMessageLogListingRef& InLogListing, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		LogListing = InLogListing;
		
		SMultiColumnTableRow<IMessageLogListingPtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	/**
	 * Generates the widget for the specified column.
	 *
	 * @param ColumnName The name of the column to generate the widget for.
	 * @return The widget.
	 */
	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		if (ColumnName == TEXT("Name"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(this, &SMessageLogCategoryListRow::HandleTextColorAndOpacity)
						.Text(this, &SMessageLogCategoryListRow::HandleNameColumnText)
				];
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	// Callback for getting the text in the 'Entries' column.
	FText HandleNameColumnText( ) const
	{
		int32 NumMessages = LogListing->GetFilteredMessages().Num();

		if (NumMessages == 0)
		{
			return LogListing->GetLabel();
		}

		return FText::Format(LOCTEXT("ColumnNameCountFormat", "{0} ({1})"), LogListing->GetLabel(), FText::AsNumber(NumMessages));		
	}

	// Callback for getting the text color.
	FSlateColor HandleTextColorAndOpacity( ) const
	{
		if (true)
		{
			return FSlateColor::UseForeground();
		}

		return FSlateColor::UseSubduedForeground();
	}

private:

	// Holds the log listing used to populate this row.
	IMessageLogListingPtr LogListing;
};


#undef LOCTEXT_NAMESPACE
