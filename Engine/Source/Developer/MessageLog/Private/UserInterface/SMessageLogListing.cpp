// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/SMessageLogListing.h"
#include "MessageFilter.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "UserInterface/SMessageLogMessageListRow.h"
#include "Framework/Commands/GenericCommands.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Styling/StyleColors.h"
#include "Logging/MessageLog.h"


#define LOCTEXT_NAMESPACE "Developer.MessageLog"


SMessageLogListing::SMessageLogListing()
	: UICommandList( MakeShareable( new FUICommandList ) )
	, bUpdatingSelection( false )
{ }


SMessageLogListing::~SMessageLogListing()
{
	MessageLogListingViewModel->OnDataChanged().RemoveAll( this );
	MessageLogListingViewModel->OnSelectionChanged().RemoveAll( this );
}


void SMessageLogListing::Construct( const FArguments& InArgs, const TSharedRef< IMessageLogListing >& InModelView )
{
	MessageLogListingViewModel = StaticCastSharedRef<FMessageLogListingViewModel>(InModelView);

	TSharedPtr<SHorizontalBox> HorizontalBox = NULL;
	TSharedPtr<SVerticalBox> VerticalBox = NULL;

	TSharedRef<SScrollBar> ScrollBar = SNew(SScrollBar);

	MessageListView = SNew(SListView<TSharedRef<FTokenizedMessage>>)
		.ListItemsSource(&MessageLogListingViewModel->GetFilteredMessages())
		.OnGenerateRow(this, &SMessageLogListing::MakeMessageLogListItemWidget)
		.OnSelectionChanged(this, &SMessageLogListing::OnLineSelectionChanged)
		.ExternalScrollbar(ScrollBar)
		.ConsumeMouseWheel(EConsumeMouseWheel::Always)
		.OnContextMenuOpening(this, &SMessageLogListing::ContextMenuOpening);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.Padding(6.0f)
		[
			SAssignNew(VerticalBox, SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNew(SScrollBox)
					.Orientation(EOrientation::Orient_Horizontal)
					+ SScrollBox::Slot()
					.FillSize(1.0f)
					[
						SNew(SScrollBorder, MessageListView.ToSharedRef())
						[
							MessageListView.ToSharedRef()
						]
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(FOptionalSize(16))
					[
						ScrollBar
					]
				]
			]
		]
	];

	//If we have some content below the message log, add a separator and a new box.
	if (MessageLogListingViewModel->GetShowFilters() || 
		MessageLogListingViewModel->GetShowPages() || 
		MessageLogListingViewModel->GetAllowClear() )
	{
		VerticalBox->AddSlot()
			.AutoHeight()
			.Padding(FMargin(6, 6, 0, 0))
			[
				SAssignNew(HorizontalBox, SHorizontalBox)
			];

		if (MessageLogListingViewModel->GetShowFilters())
		{
			HorizontalBox->AddSlot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Left)
				[
					SNew(SComboButton)
					.ButtonStyle(FAppStyle::Get(), "Button")
					.ForegroundColor(FStyleColors::White)
					.OnGetMenuContent(this, &SMessageLogListing::OnGetFilterMenuContent)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Show", "SHOW"))
						.ToolTipText(LOCTEXT("ShowToolTip", "Only show messages of the selected types"))
					]
				];
		}

		HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Right)
			[
				SNew(SComboButton)
				.IsEnabled(this, &SMessageLogListing::IsPageWidgetEnabled)
				.Visibility(this, &SMessageLogListing::GetPageWidgetVisibility)
				.ButtonStyle(FAppStyle::Get(), "Button")
				.ForegroundColor(FStyleColors::White)
				.OnGetMenuContent(this, &SMessageLogListing::OnGetPageMenuContent)
				.ButtonColorAndOpacity(FStyleColors::White)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &SMessageLogListing::OnGetPageMenuLabel)
					.ToolTipText(LOCTEXT("PageToolTip", "Choose the log page to view"))
				]
			];

		HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.OnClicked(this, &SMessageLogListing::OnClear)
				.IsEnabled(this, &SMessageLogListing::IsClearWidgetEnabled)
				.Visibility(this, &SMessageLogListing::GetClearWidgetVisibility)
				.ForegroundColor(FStyleColors::White)
				.ContentPadding(FMargin(6.5, 2))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ClearMessageLog", "CLEAR"))
					.ToolTipText(LOCTEXT("ClearMessageLog_ToolTip", "Clear the messages in this log"))
				]
			];
	}

	// Register with the the view object so that it will notify if any data changes
	MessageLogListingViewModel->OnDataChanged().AddSP( this, &SMessageLogListing::OnChanged );
	MessageLogListingViewModel->OnSelectionChanged().AddSP( this, &SMessageLogListing::OnSelectionChanged );

	if (FGenericCommands::IsRegistered())
	{
		UICommandList->MapAction(FGenericCommands::Get().Copy,
			FExecuteAction::CreateSP(this, &SMessageLogListing::CopySelectedToClipboard),
			FCanExecuteAction());
	}
}


void SMessageLogListing::OnChanged()
{
	ClearSelectedMessages();
	RefreshVisibility();
}


void SMessageLogListing::OnSelectionChanged()
{
	if( bUpdatingSelection )
	{
		return;
	}

	bUpdatingSelection = true;
	const auto& SelectedMessages = MessageLogListingViewModel->GetSelectedMessages();
	MessageListView->ClearSelection();
	for( auto It = SelectedMessages.CreateConstIterator(); It; ++It )
	{
		MessageListView->SetItemSelection( *It, true );
	}

	if( SelectedMessages.Num() > 0 )
	{
		ScrollToMessage( SelectedMessages[0] );
	}
	bUpdatingSelection = false;
}


void SMessageLogListing::RefreshVisibility()
{
	const TArray< TSharedRef<FTokenizedMessage> >& Messages = MessageLogListingViewModel->GetFilteredMessages();
	if (Messages.Num() > 0)
	{
		if (MessageLogListingViewModel->GetScrollToBottom())
		{
			ScrollToMessage( Messages.Last() );
		}
		else
		{
			ScrollToMessage( Messages[0] );
		}
	}
	
	MessageListView->RequestListRefresh();
}


void SMessageLogListing::BroadcastMessageTokenClicked( TSharedPtr<FTokenizedMessage> Message, const TSharedRef<IMessageToken>& Token )
{
	ClearSelectedMessages();
	SelectMessage( Message.ToSharedRef(), true );
	MessageLogListingViewModel->ExecuteToken(Token);
}

void SMessageLogListing::BroadcastMessageDoubleClicked(TSharedPtr< class FTokenizedMessage > Message)
{
	if (Message->GetMessageTokens().Num() > 0)
	{
		TSharedPtr<IMessageToken> MessageLink = Message->GetMessageLink();
		if (MessageLink.IsValid())
		{
			MessageLogListingViewModel->ExecuteToken(MessageLink->AsShared());
		}
	}	
}

const TArray< TSharedRef<FTokenizedMessage> > SMessageLogListing::GetSelectedMessages() const
{
	return MessageLogListingViewModel->GetSelectedMessages();
}


void SMessageLogListing::SelectMessage( const TSharedRef<class FTokenizedMessage>& Message, bool bSelected ) const
{
	MessageLogListingViewModel->SelectMessage( Message, bSelected );
}


bool SMessageLogListing::IsMessageSelected( const TSharedRef<class FTokenizedMessage>& Message ) const
{
	return MessageLogListingViewModel->IsMessageSelected( Message );
}


void SMessageLogListing::ScrollToMessage( const TSharedRef<class FTokenizedMessage>& Message ) const
{
	if(!MessageListView->IsItemVisible(Message))
	{
		MessageListView->RequestScrollIntoView( Message );
	}
}


void SMessageLogListing::ClearSelectedMessages() const
{
	MessageLogListingViewModel->ClearSelectedMessages();
}


void SMessageLogListing::InvertSelectedMessages() const
{
	MessageLogListingViewModel->InvertSelectedMessages();
}


FString SMessageLogListing::GetSelectedMessagesAsString() const
{
	return MessageLogListingViewModel->GetSelectedMessagesAsString();
}


FString SMessageLogListing::GetAllMessagesAsString() const
{
	return MessageLogListingViewModel->GetAllMessagesAsString();
}


TSharedRef<ITableRow> SMessageLogListing::MakeMessageLogListItemWidget( TSharedRef<FTokenizedMessage> Message, const TSharedRef<STableViewBase>& OwnerTable )
{
	return
		SNew(SMessageLogMessageListRow, OwnerTable)
		.Message(Message)
		.OnTokenClicked( this, &SMessageLogListing::BroadcastMessageTokenClicked )
		.OnMessageDoubleClicked( this, &SMessageLogListing::BroadcastMessageDoubleClicked );
}

static bool CanMessageTokenBeActor(EMessageToken::Type TokenType)
{
	return (TokenType == EMessageToken::Actor || TokenType == EMessageToken::Object || TokenType == EMessageToken::AssetData);
}

static void ActivateMessageToken(const TSharedRef<FTokenizedMessage>& Message)
{
	const TArray<TSharedRef<IMessageToken>>& MessageTokens = Message->GetMessageTokens();
	for (const TSharedRef<IMessageToken>& Token : MessageTokens)
	{
		if (CanMessageTokenBeActor(Token->GetType()))
		{
			const FOnMessageTokenActivated& ActivateDelegate = Token->GetOnMessageTokenActivated();
			if (ActivateDelegate.IsBound())
			{
				ActivateDelegate.Execute(Token);
				return;
			}
		}
	}
}

// The source string for a formatted message is the original format specifier string, without any substitutions.  Optional FormatKey
// can be used to also return the value of a specific field in the formatted string.
static FString GetSourceStringFromMessage(const TArray<TSharedRef<IMessageToken>>& MessageTokens, const TCHAR* FormatKey, FString& OutFormatValue)
{
	OutFormatValue.Empty();

	const TSharedRef<IMessageToken>* TextToken = MessageTokens.FindByPredicate([](const TSharedRef<IMessageToken>& Token) { return Token->GetType() == EMessageToken::Text; });
	if (TextToken)
	{
		const FText& Text = (*TextToken)->ToText();
		TArray<FHistoricTextFormatData> TextHistory;
		FTextInspector::GetHistoricFormatData(Text, TextHistory);

		if (TextHistory.Num() > 0)
		{
			if (FormatKey)
			{
				const FFormatArgumentValue* FoundValue = TextHistory[0].Arguments.Find(FormatKey);
				if (FoundValue)
				{
					OutFormatValue = FoundValue->GetTextValue().ToString();
				}
			}

			return TextHistory[0].SourceFmt.GetSourceString();
		}
	}

	return FString();
}

void SMessageLogListing::ContextMenuSelect() const
{
	FMessageLog::OnMultiSelectActorBegin().ExecuteIfBound();
	TArray<TSharedRef<FTokenizedMessage>> SelectedMessages = GetSelectedMessages();
	for (TSharedRef<FTokenizedMessage>& Message : SelectedMessages)
	{
		ActivateMessageToken(Message);
	}
	FMessageLog::OnMultiSelectActorEnd().ExecuteIfBound();
}

// Optional FormatKey requires the value of the given field in the format specifier to match across all messages.
void SMessageLogListing::ContextMenuSelectByMessage(const TCHAR* FormatKey) const
{
	TArray<TSharedRef<FTokenizedMessage>> SelectedMessages = GetSelectedMessages();
	if (SelectedMessages.Num() == 1)
	{
		TSharedRef<FTokenizedMessage>& SelectedMessage = SelectedMessages[0];
		const TArray<TSharedRef<IMessageToken>>& SelectedMessageTokens = SelectedMessage->GetMessageTokens();
		FString SelectedFormatValue;
		FString SelectedSourceString = GetSourceStringFromMessage(SelectedMessageTokens, FormatKey, SelectedFormatValue);

		if (!SelectedSourceString.IsEmpty() && (FormatKey == nullptr || !SelectedFormatValue.IsEmpty()) &&
			SelectedMessageTokens.ContainsByPredicate([](const TSharedRef<IMessageToken>& Token) { return CanMessageTokenBeActor(Token->GetType()); }))
		{
			FMessageLog::OnMultiSelectActorBegin().ExecuteIfBound();
			for (const TSharedRef<FTokenizedMessage>& MessageItem : MessageListView->GetItems())
			{
				const TArray<TSharedRef<IMessageToken>>& MessageItemTokens = MessageItem->GetMessageTokens();
				FString MessageItemFormatValue;
				FString MessageItemSourceString = GetSourceStringFromMessage(MessageItemTokens, FormatKey, MessageItemFormatValue);

				if (MessageItemSourceString == SelectedSourceString && MessageItemFormatValue == SelectedFormatValue)
				{
					ActivateMessageToken(MessageItem);
				}
			}
			FMessageLog::OnMultiSelectActorEnd().ExecuteIfBound();
		}
	}
}

TSharedPtr<SWidget> SMessageLogListing::ContextMenuOpening()
{
	// Only show actor selection context menu for map check message log
	if (MessageLogListingViewModel->GetLabel().BuildSourceString() == TEXT("Map Check"))
	{
		// Search for a message with an actor, object, or asset token
		TArray<TSharedRef<FTokenizedMessage>> SelectedMessages = GetSelectedMessages();
		for (TSharedRef<FTokenizedMessage>& Message : SelectedMessages)
		{
			const TArray<TSharedRef<IMessageToken>>& MessageTokens = Message->GetMessageTokens();
			if (MessageTokens.ContainsByPredicate([](const TSharedRef<IMessageToken>& Token) { return CanMessageTokenBeActor(Token->GetType()); }))
			{
				// Found a relevant message, start building the context menu
				FMenuBuilder MenuBuilder(true, nullptr);

				// Batch selection by message type or mesh name is only supported for single selection, for simplicity and clarity
				if (SelectedMessages.Num() == 1)
				{
					MenuBuilder.AddMenuEntry(LOCTEXT("SelectActor", "Select actor"), LOCTEXT("SelectActorTooltip", "Select actor referenced by message"), FSlateIcon(),
						FUIAction(FExecuteAction::CreateRaw(this, &SMessageLogListing::ContextMenuSelect)));

					const TSharedRef<IMessageToken>* TextToken = MessageTokens.FindByPredicate([](const TSharedRef<IMessageToken>& Token) { return Token->GetType() == EMessageToken::Text; });
					if (TextToken)
					{
						const FText& Text = (*TextToken)->ToText();
						TArray<FHistoricTextFormatData> TextHistory;
						FTextInspector::GetHistoricFormatData(Text, TextHistory);

						if (TextHistory.Num() > 0)
						{
							MenuBuilder.AddMenuEntry(LOCTEXT("SelectActorsByMessage", "Select actors by message"), LOCTEXT("SelectActorsByMessageTooltip", "Select actors with the same map check message"), FSlateIcon(),
								FUIAction(FExecuteAction::CreateRaw(this, &SMessageLogListing::ContextMenuSelectByMessage, (const TCHAR*)nullptr)));

							if (TextHistory[0].Arguments.Contains(TEXT("MeshName")))
							{
								MenuBuilder.AddMenuEntry(LOCTEXT("SelectActorsByMessageAndMesh", "Select actors by message / mesh"), LOCTEXT("SelectActorsByMessageAndMeshTooltip", "Select actors with the same map check message and mesh"), FSlateIcon(),
									FUIAction(FExecuteAction::CreateRaw(this, &SMessageLogListing::ContextMenuSelectByMessage, (const TCHAR*)TEXT("MeshName"))));
							}
						}
					}
				}
				else
				{
					MenuBuilder.AddMenuEntry(LOCTEXT("SelectActors", "Select actors"), LOCTEXT("SelectActorsTooltip", "Select actors referenced by all selected messages"), FSlateIcon(),
						FUIAction(FExecuteAction::CreateRaw(this, &SMessageLogListing::ContextMenuSelect)));
				}

				return MenuBuilder.MakeWidget();
			}
		}
	}

	return nullptr;
}

void SMessageLogListing::OnLineSelectionChanged( TSharedPtr< FTokenizedMessage > Selection, ESelectInfo::Type /*SelectInfo*/ )
{
	if (bUpdatingSelection)
	{
		return;
	}

	bUpdatingSelection = true;
	{
		MessageLogListingViewModel->SelectMessages(MessageListView->GetSelectedItems());
	}
	bUpdatingSelection = false;
}


void SMessageLogListing::CopySelectedToClipboard() const
{
	CopyLogOutput( true, true );
}


FString SMessageLogListing::CopyLogOutput( bool bSelected, bool bClipboard ) const
{
	FString CombinedString;

	if( bSelected )
	{
		// Get the selected item and then get the selected messages as a string.
		CombinedString = GetSelectedMessagesAsString();
	}
	else
	{
		// Get the selected item and then get all the messages as a string.
		CombinedString = GetAllMessagesAsString();
	}
	if( bClipboard )
	{
		// Pass that to the clipboard.
		FPlatformApplicationMisc::ClipboardCopy( *CombinedString );
	}

	return CombinedString;
}


const TSharedRef< const FUICommandList > SMessageLogListing::GetCommandList() const 
{ 
	return UICommandList;
}


FReply SMessageLogListing::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	return UICommandList->ProcessCommandBindings( InKeyEvent ) ? FReply::Handled() : FReply::Unhandled();
}


EVisibility SMessageLogListing::GetFilterMenuVisibility()
{
	if( MessageLogListingViewModel->GetShowFilters() )
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}


TSharedRef<ITableRow> SMessageLogListing::MakeShowWidget(TSharedRef<FMessageFilter> Selection, const TSharedRef<STableViewBase>& OwnerTable)
{
	return
		SNew(STableRow< TSharedRef<FMessageFilter> >, OwnerTable)
		[
			SNew(SCheckBox)
			.IsChecked(Selection, &FMessageFilter::OnGetDisplayCheckState)
			.OnCheckStateChanged(Selection, &FMessageFilter::OnDisplayCheckStateChanged)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot().AutoWidth()
				[
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.WidthOverride(16)
					.HeightOverride(16)
					[
						SNew(SImage)
						.Image(Selection->GetIcon().GetIcon())
					]
				]
				+SHorizontalBox::Slot().AutoWidth()
				[
					SNew(STextBlock)
					.Text(Selection->GetName())
				]
			]
		];
}


TSharedRef<SWidget> SMessageLogListing::OnGetFilterMenuContent()
{
	return 
		SNew( SListView< TSharedRef<FMessageFilter> >)
		.ListItemsSource(&MessageLogListingViewModel->GetMessageFilters())
		.OnGenerateRow(this, &SMessageLogListing::MakeShowWidget);
}


FText SMessageLogListing::OnGetPageMenuLabel() const
{
	if(MessageLogListingViewModel->GetPageCount() > 1)
	{
		return MessageLogListingViewModel->GetPageTitle(MessageLogListingViewModel->GetCurrentPageIndex());
	}
	else
	{
		return LOCTEXT("PageMenuLabel", "PAGE");
	}
}


TSharedRef<SWidget> SMessageLogListing::OnGetPageMenuContent() const
{
	if(MessageLogListingViewModel->GetPageCount() > 1)
	{
		FMenuBuilder MenuBuilder(true, NULL);
		for(uint32 PageIndex = 0; PageIndex < MessageLogListingViewModel->GetPageCount(); PageIndex++)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("PageName"), MessageLogListingViewModel->GetPageTitle(PageIndex));
			MenuBuilder.AddMenuEntry( 
				MessageLogListingViewModel->GetPageTitle(PageIndex), 
				FText::Format(LOCTEXT("PageMenuEntry_Tooltip", "View page: {PageName}"), Arguments), 
				FSlateIcon(),
				FExecuteAction::CreateSP(const_cast<SMessageLogListing*>(this), &SMessageLogListing::OnPageSelected, PageIndex));
		}

		return MenuBuilder.MakeWidget();
	}

	return SNullWidget::NullWidget;
}


void SMessageLogListing::OnPageSelected(uint32 PageIndex)
{
	MessageLogListingViewModel->SetCurrentPageIndex(PageIndex);
}


bool SMessageLogListing::IsPageWidgetEnabled() const
{
	return MessageLogListingViewModel->GetPageCount() > 1;
}


EVisibility SMessageLogListing::GetPageWidgetVisibility() const
{
	if(MessageLogListingViewModel->GetShowPages())
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}


bool SMessageLogListing::IsClearWidgetEnabled() const
{
	return MessageLogListingViewModel->NumMessages() > 0;
}


EVisibility SMessageLogListing::GetClearWidgetVisibility() const
{
	if (MessageLogListingViewModel->GetAllowClear())
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}


FReply SMessageLogListing::OnClear()
{
	MessageLogListingViewModel->ClearMessages();
	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
