// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/SMessageLogMessageListRow.h"
#include "Widgets/SToolTip.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "Misc/UObjectToken.h"
#include "Widgets/Input/SHyperlink.h"
#include "Internationalization/Regex.h"
#include "Styling/StyleColors.h"
#include "AssetRegistry/AssetDataToken.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/DataValidation/Fixer.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "SMessageLogMessageListRow"




TSharedRef<SWidget> SMessageLogMessageListRow::CreateHyperlink(const TSharedRef<IMessageToken>& InMessageToken, const FText& InToolTip /*= FText() */)
{
	return SNew(SHyperlink)
		.Text(InMessageToken->ToText())
		.ToolTipText(InToolTip)
		.TextStyle(FAppStyle::Get(), "MessageLog")
		.OnNavigate(this, &SMessageLogMessageListRow::HandleHyperlinkNavigate, InMessageToken);
}

void SMessageLogMessageListRow::Construct(const FArguments& InArgs, const TSharedRef< STableViewBase >& InOwnerTableView)
{
	this->OnTokenClicked = InArgs._OnTokenClicked;
	this->OnMessageDoubleClicked = InArgs._OnMessageDoubleClicked;

	Message = InArgs._Message.Get();

	STableRow<TSharedPtr<FTokenizedMessage>>::Construct(
		STableRow<TSharedPtr<FTokenizedMessage>>::FArguments()
		.Content()
		[
			GenerateWidget()
		],
		InOwnerTableView
		);
}

TSharedRef<SWidget> SMessageLogMessageListRow::GenerateWidget()
{
	// See if we have any valid tokens which match the column name
	const TArray<TSharedRef<IMessageToken>>& MessageTokens = Message->GetMessageTokens();

	// Create the horizontal box and add the icon
	TSharedRef<SHorizontalBox> MessageBox = SNew(SHorizontalBox);
	TSharedRef<SHorizontalBox> LinkBox = SNew(SHorizontalBox);
	TSharedRef<SVerticalBox> FixBox = SNew(SVerticalBox);
	FName SeverityImageName = NAME_None;
	bool HasLinks = false;
	bool bHasFixes = false;

	// Iterate over parts of the message and create widgets for them
	for (auto TokenIt = MessageTokens.CreateConstIterator(); TokenIt; ++TokenIt)
	{
		const TSharedRef<IMessageToken>& Token = *TokenIt;

		switch (Token->GetType())
		{
		case EMessageToken::Severity:
		{
			if (SeverityImageName == NAME_None)
			{
				const TSharedRef<FSeverityToken> SeverityToken = StaticCastSharedRef<FSeverityToken>(Token);
				SeverityImageName = FTokenizedMessage::GetSeverityIconName(SeverityToken->GetSeverity());
			}
		}
			break;

		case EMessageToken::Documentation:
		case EMessageToken::Tutorial:
		{
			CreateMessage(LinkBox, Token, 10.0f);
			HasLinks = true;
		}
			break;

		case EMessageToken::Fix:
		{
			TSharedRef<SHorizontalBox> RowWrapper = SNew(SHorizontalBox);
			CreateMessage(RowWrapper, Token, 2.0f);
			FixBox->AddSlot()
			.AutoHeight()
			.Padding(24.0f, 0.0f, 0.0f, 0.0f)
			[
				RowWrapper
			];
			bHasFixes = true;
		}
			break;

		default:
			CreateMessage(MessageBox, Token, 2.0f);
			break;
		}
	}

	TSharedRef<SHorizontalBox> FullMessageBox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Top)
		[
			SNew(SBox)
			.Padding((Message->GetIndentationLevel() * 20.f) + 2.f, 2.f, 2.f, 2.f)
			[
				(SeverityImageName == NAME_None)
				? SNullWidget::NullWidget
				: static_cast<TSharedRef<SWidget>>(
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.WidthOverride(16)
					.HeightOverride(16)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush(SeverityImageName))
					])
			]
		]

	+ SHorizontalBox::Slot()
	.FillWidth(1.0f)
	.VAlign(VAlign_Top)
	[
		MessageBox
	]

	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Top)
	.Padding(1.0f)
	[
		!HasLinks
		? SNullWidget::NullWidget
		: static_cast<TSharedRef<SWidget>>(SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.Padding(FMargin(0.0f, 1.0f, 10.0f, 1.0f))
		[
			LinkBox
		])
	];

	if (bHasFixes)
	{
		return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			FullMessageBox
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			FixBox
		];
	}
	else
	{
		return FullMessageBox;
	}

}

void SMessageLogMessageListRow::CreateMessage(const TSharedRef<SHorizontalBox>& InHorzBox, const TSharedRef<IMessageToken>& InMessageToken, float Padding)
{
	TSharedPtr<SWidget> RowContent;
	FName IconBrushName;

	TAttribute<EVisibility> TokenContentVisbility;
	
	switch (InMessageToken->GetType())
	{
	case EMessageToken::Image:
	{
		const TSharedRef<FImageToken> ImageToken = StaticCastSharedRef<FImageToken>(InMessageToken);

		if (ImageToken->GetImageName() != NAME_None)
		{
			if (InMessageToken->GetOnMessageTokenActivated().IsBound())
			{
				RowContent = SNew(SButton)
					.OnClicked(this, &SMessageLogMessageListRow::HandleTokenButtonClicked, InMessageToken)
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush(ImageToken->GetImageName()))
					];
			}
			else
			{
				RowContent = SNew(SImage)
					.Image(FAppStyle::Get().GetBrush(ImageToken->GetImageName()));
			}
		}
	}
		break;

	case EMessageToken::AssetData:
	{
		const TSharedRef<FAssetDataToken> AssetDataToken = StaticCastSharedRef<FAssetDataToken>(InMessageToken);

		IconBrushName = FName("Icons.Search");
		RowContent = CreateHyperlink(InMessageToken, FAssetDataToken::DefaultOnGetAssetDisplayName().IsBound()
			? FAssetDataToken::DefaultOnGetAssetDisplayName().Execute(AssetDataToken->GetAssetData(), true)
			: InMessageToken->ToText());
	}
		break;
	case EMessageToken::Object:
	{
		const TSharedRef<FUObjectToken> UObjectToken = StaticCastSharedRef<FUObjectToken>(InMessageToken);

		IconBrushName = FName("Icons.Search");

		UObject* Object = nullptr;

		// Due to blueprint reconstruction, we can't directly use the Object as it will get trashed during the blueprint reconstruction and the message token will no longer point to the right UObject.
		// Instead we will retrieve the object from the name which should always be good.
		if (UObjectToken->GetObject().IsValid())
		{
			if (!UObjectToken->ToText().ToString().Equals(UObjectToken->GetObject().Get()->GetName()))
			{
				Object = FindObject<UObject>(nullptr, *UObjectToken->GetOriginalObjectPathName());
			}
			else
			{
				Object = const_cast<UObject*>(UObjectToken->GetObject().Get());
			}
		}
		else
		{
			// We have no object (probably because is now stale), try finding the original object linked to this message token to see if it still exist
			Object = FindObject<UObject>(nullptr, *UObjectToken->GetOriginalObjectPathName());
		}

		RowContent = CreateHyperlink(InMessageToken, FUObjectToken::DefaultOnGetObjectDisplayName().IsBound()
			? FUObjectToken::DefaultOnGetObjectDisplayName().Execute(Object, true)
			: UObjectToken->ToText());
	}
		break;

	case EMessageToken::URL:
	{
		const TSharedRef<FURLToken> URLToken = StaticCastSharedRef<FURLToken>(InMessageToken);

		IconBrushName = FName("MessageLog.Url");
		RowContent = CreateHyperlink(InMessageToken, FText::FromString(URLToken->GetURL()));
	}
		break;
	case EMessageToken::EdGraph:
	{
		IconBrushName = FName("Icons.Search");
		RowContent = CreateHyperlink(InMessageToken, InMessageToken->ToText());
	}
	break;
	case EMessageToken::Action:
	{
		const TSharedRef<FActionToken> ActionToken = StaticCastSharedRef<FActionToken>(InMessageToken);

		IconBrushName = FName("MessageLog.Action");
		RowContent = SNew(SHyperlink)
			.Text(InMessageToken->ToText())
			.ToolTipText(ActionToken->GetActionDescription())
			.TextStyle(FAppStyle::Get(), "MessageLog")
			.IsEnabled_Raw(this, &SMessageLogMessageListRow::GetActionLinkEnable, ActionToken)
			.OnNavigate(this, &SMessageLogMessageListRow::HandleActionHyperlinkNavigate, ActionToken);

		TokenContentVisbility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateRaw(this, &SMessageLogMessageListRow::GetActionLinkVisibility, ActionToken));
	}
		break;

	case EMessageToken::AssetName:
	{
		const TSharedRef<FAssetNameToken> AssetNameToken = StaticCastSharedRef<FAssetNameToken>(InMessageToken);

		IconBrushName = FName("Icons.Search");
		RowContent = CreateHyperlink(InMessageToken, AssetNameToken->ToText());
	}
		break;

	case EMessageToken::DynamicText:
	{
			const TSharedRef<FDynamicTextToken> TextToken = StaticCastSharedRef<FDynamicTextToken>(InMessageToken);
			RowContent = SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text(TextToken->GetTextAttribute());
	}
		break;

	case EMessageToken::Text:
	{
		if (InMessageToken->GetOnMessageTokenActivated().IsBound())
		{
			RowContent = CreateHyperlink(InMessageToken, InMessageToken->ToText());
		}
		else
		{
			FString MessageString = InMessageToken->ToText().ToString();
			const TSharedRef<FTextToken> TextToken = StaticCastSharedRef<FTextToken>(InMessageToken);

			// ^((?:[\w]\:|\\)(?:(?:\\[a-z_\-\s0-9\.]+)+)\.(?:cpp|h))\((\d+)\)
			// https://regex101.com/r/ccAnMS/1
			FRegexPattern FileAndLinePattern(TEXT("^((?:[\\w]\\:|\\\\)(?:(?:\\\\[A-Za-z_\\-\\s0-9\\.]+)+)\\.(?:cpp|h))\\((\\d+)\\)"));
			if (!TextToken->IsSourceLinkOnLeft())
			{
				FileAndLinePattern = FRegexPattern(TEXT("((?:[\\w]\\:|\\\\)(?:(?:\\\\[A-Za-z_\\-\\s0-9\\.]+)+)\\.(?:cpp|h))\\((\\d+)\\)$"));
			}

			FRegexMatcher FileAndLineRegexMatcher(FileAndLinePattern, MessageString);

			TSharedRef<SWidget> SourceLink = SNullWidget::NullWidget;

			if ( FileAndLineRegexMatcher.FindNext() )
			{
				FString FileName = FileAndLineRegexMatcher.GetCaptureGroup(1);
				int32 LineNumber = FCString::Atoi(*FileAndLineRegexMatcher.GetCaptureGroup(2));

				// Remove the hyperlink from the message, since we're splitting it into its own string.
				if (TextToken->IsSourceLinkOnLeft())
				{
					MessageString.RightChopInline(FileAndLineRegexMatcher.GetMatchEnding(), EAllowShrinking::No);
				}
				else
				{
					MessageString.LeftChopInline(FileAndLineRegexMatcher.GetCaptureGroup(0).Len(), EAllowShrinking::No);
				}

				SourceLink = SNew(SHyperlink)
					.Style(FAppStyle::Get(), "Common.GotoNativeCodeHyperlink")
					.TextStyle(FAppStyle::Get(), "MessageLog")
					.OnNavigate_Lambda([=] { FSlateApplication::Get().GotoLineInSource(FileName, LineNumber); })
					.Text(FText::FromString(FileAndLineRegexMatcher.GetCaptureGroup(0)));
			}

			if (TextToken->IsSourceLinkOnLeft())
			{
				RowContent = SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0)
					[
						SourceLink
					]
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(0.f, 4.f))
					[
						SNew(STextBlock)
						.Text(FText::FromString(MessageString))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.TextStyle(FAppStyle::Get(), "MessageLog")
					];
			}
			else
			{
				RowContent = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(0.f, 4.f))
					[
						SNew(STextBlock)
						.Text(FText::FromString(MessageString))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.TextStyle(FAppStyle::Get(), "MessageLog")
					]
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0)
					[
						SourceLink
					];
			}

		}
	}
		break;

#if WITH_EDITOR
	case EMessageToken::Documentation:
	{
		const TSharedRef<FDocumentationToken> DocumentationToken = StaticCastSharedRef<FDocumentationToken>(InMessageToken);

		IconBrushName = FName("MessageLog.Docs");
		RowContent = SNew(SHyperlink)
			.Text(LOCTEXT("DocsLabel", "Docs"))
			.ToolTip(IDocumentation::Get()->CreateToolTip(
			LOCTEXT("DocumentationTokenToolTip", "Click to open documentation"),
			NULL,
			DocumentationToken->GetPreviewExcerptLink(),
			DocumentationToken->GetPreviewExcerptName())
			)
			.TextStyle(FAppStyle::Get(), "MessageLog")
			.OnNavigate(this, &SMessageLogMessageListRow::HandleDocsHyperlinkNavigate, DocumentationToken->GetDocumentationLink());
	}
		break;

	case EMessageToken::Tutorial:
	{
		const TSharedRef<FTutorialToken> TutorialToken = StaticCastSharedRef<FTutorialToken>(InMessageToken);

		IconBrushName = FName("MessageLog.Tutorial");
		RowContent = SNew(SHyperlink)
			.Text(LOCTEXT("TutorialLabel", "Tutorial"))
			.ToolTipText(LOCTEXT("TutorialTokenToolTip", "Click to open tutorial"))
			.TextStyle(FAppStyle::Get(), "MessageLog")
			.OnNavigate(this, &SMessageLogMessageListRow::HandleTutorialHyperlinkNavigate, TutorialToken->GetTutorialAssetName());
	}
		break;
#endif

	case EMessageToken::Actor:
	{
		const TSharedRef<FActorToken> ActorToken = StaticCastSharedRef<FActorToken>(InMessageToken);

		IconBrushName = FName("Icons.Search");
		RowContent = CreateHyperlink(InMessageToken, ActorToken->ToText());
	}
		break;

	case EMessageToken::Fix:
	{
		TSharedRef<FFixToken> FixToken = StaticCastSharedRef<FFixToken>(InMessageToken);

		RowContent =
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 2.0f, 4.0f, 2.0f)
				[
					SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image_Lambda([FixToken]
						{
							switch (FixToken->GetFixer()->GetApplicability(FixToken->GetFixIndex()))
							{
								case EFixApplicability::CanBeApplied:
									return FAppStyle::Get().GetBrush("MessageLog.Fix");
								case EFixApplicability::Applied:
									return FAppStyle::Get().GetBrush("Icons.Check");
								default:
									return FAppStyle::Get().GetBrush("Icons.BulletPoint16");
							}
						})
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SHyperlink)
						.Text(InMessageToken->ToText())
						.TextStyle(FAppStyle::Get(), "MessageLog")
						.IsEnabled_Lambda(
							[FixToken]
							{ return FixToken->GetFixer()->GetApplicability(FixToken->GetFixIndex()) == EFixApplicability::CanBeApplied; }
						)
						.OnNavigate(FSimpleDelegate::CreateSPLambda(
							AsShared(),
							[this, FixToken]
							{
								FFixResult Result = FixToken->GetFixer()->ApplyFix(FixToken->GetFixIndex());
								FNotificationInfo NotificationInfo(
									Result.bIsSuccess ? LOCTEXT("FixAppliedSuccessfully", "Fix applied successfully")
													  : LOCTEXT("FailedToApplyFix", "Failed to apply fix")
								);
								NotificationInfo.SubText = Result.Message;
								NotificationInfo.bUseThrobber = false;
								FSlateNotificationManager::Get()
									.AddNotification(NotificationInfo)
									->SetCompletionState(
										Result.bIsSuccess ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail
									);
							}
						))
				];
	}
		break;
	}

	if (RowContent.IsValid())
	{
		InHorzBox->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(Padding, 0.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)
				.Visibility(TokenContentVisbility.IsBound() ? TokenContentVisbility : EVisibility::Visible)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					(IconBrushName == NAME_None)
					? SNullWidget::NullWidget
					: static_cast<TSharedRef<SWidget>>(SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush(IconBrushName)))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f, 0.0f, 0.0f)
				[
					RowContent.ToSharedRef()
				]
			];
	}
}

#undef LOCTEXT_NAMESPACE
