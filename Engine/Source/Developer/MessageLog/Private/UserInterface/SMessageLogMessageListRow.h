// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Logging/TokenizedMessage.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#if WITH_EDITOR
	#include "IDocumentation.h"
	#include "Interfaces/IMainFrameModule.h"
#endif

#define LOCTEXT_NAMESPACE "SMessageLogMessageListRow"

class SMessageLogMessageListRow
	: public STableRow<TSharedPtr<FTokenizedMessage>>
{
public:

	DECLARE_DELEGATE_TwoParams( FOnTokenClicked, TSharedPtr<FTokenizedMessage>, const TSharedRef<class IMessageToken>& );
	DECLARE_DELEGATE_OneParam( FOnMessageClicked, TSharedPtr<FTokenizedMessage> );

public:

	SLATE_BEGIN_ARGS(SMessageLogMessageListRow)
		: _Message()
		, _OnTokenClicked()
		, _OnMessageDoubleClicked()
	{ }
		SLATE_ATTRIBUTE(TSharedPtr<FTokenizedMessage>, Message)
		SLATE_EVENT(FOnTokenClicked, OnTokenClicked)
		SLATE_EVENT(FOnMessageClicked, OnMessageDoubleClicked)
	SLATE_END_ARGS()

	/**
	 * Construct child widgets that comprise this widget.
	 *
	 * @param InArgs  Declaration from which to construct this widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef< STableViewBase >& InOwnerTableView );

public:

	/** @return Widget for this log listing item*/
	virtual TSharedRef<SWidget> GenerateWidget();

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		FReply HandledReply = STableRow<TSharedPtr<FTokenizedMessage>>::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);

		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			OnMessageDoubleClicked.ExecuteIfBound(Message);
			HandledReply = FReply::Handled();
		}
		return HandledReply;
	}

protected:

	TSharedRef<SWidget> CreateHyperlink( const TSharedRef<IMessageToken>& InMessageToken, const FText& InToolTip = FText() );

	void CreateMessage( const TSharedRef<SHorizontalBox>& InHorzBox, const TSharedRef<IMessageToken>& InMessageToken, float Padding );

private:

	EVisibility GetActionLinkVisibility(TSharedRef<FActionToken> ActionToken) const
	{
		return ActionToken->IsValidAction() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	bool GetActionLinkEnable(TSharedRef<FActionToken> ActionToken) const
	{
		return ActionToken->CanExecuteAction();
	}

	void HandleActionHyperlinkNavigate( TSharedRef<FActionToken> ActionToken )
	{
		ActionToken->ExecuteAction();
	}

	void HandleHyperlinkNavigate( TSharedRef<IMessageToken> InMessageToken )
	{
		InMessageToken->GetOnMessageTokenActivated().ExecuteIfBound(InMessageToken);
		OnTokenClicked.ExecuteIfBound(Message, InMessageToken);
	}

	FReply HandleTokenButtonClicked( TSharedRef<IMessageToken> InMessageToken )
	{
		InMessageToken->GetOnMessageTokenActivated().ExecuteIfBound(InMessageToken);
		OnTokenClicked.ExecuteIfBound(Message, InMessageToken);
		return FReply::Handled();
	}

#if WITH_EDITOR
	void HandleDocsHyperlinkNavigate( FString DocumentationLink )
	{
		IDocumentation::Get()->Open(DocumentationLink, FDocumentationSourceInfo(TEXT("msg_log")));
	}

	void HandleTutorialHyperlinkNavigate( FString TutorialAssetName )
	{
		IMainFrameModule& MainFrameModule = FModuleManager::GetModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		MainFrameModule.BroadcastMainFrameRequestResource("Tutorial", TutorialAssetName);
	}
#endif

protected:

	/** The message used to create this widget. */
	TSharedPtr<FTokenizedMessage> Message;

	/** Delegate to execute when the token is clicked. */
	FOnTokenClicked OnTokenClicked;

	/** Delegate to execute when the message is double-clicked. */
	FOnMessageClicked OnMessageDoubleClicked;
};


#undef LOCTEXT_NAMESPACE
