// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/IMessageLog.h"

class IMessageLogListing;

/** Type definition for shared pointers to instances of IMessageLogListing. */
typedef TSharedPtr<class IMessageLogListing> IMessageLogListingPtr;

/** Type definition for shared references to instances of IMessageLogListing. */
typedef TSharedRef<class IMessageLogListing> IMessageLogListingRef;


/**
 * A message log listing, such as the Compiler Log, or the Map Check Log.
 */
class IMessageLogListing : public IMessageLog 
{
public:
	
	/** Clears messages (if paged, in the current page) */
	virtual void ClearMessages() = 0;

	/** 
	 * Gets a list of the selected messages for the specific log listing
	 * 
	 * @returns an array of selected messages.
	 */
	virtual const TArray< TSharedRef<class FTokenizedMessage> >& GetSelectedMessages() const = 0;

	/** 
	 * Sets multiple selected messages 
	 *
	 * @param	InSelectedMessages	The messages to select.
	 */
	virtual void SelectMessages( const TArray< TSharedRef<class FTokenizedMessage> >& InSelectedMessages ) = 0;

	/** 
	 * Gets a list of the filtered messages for the specific log listing 
	 *
	 * @returns an array of filtered messages.
	 */
	virtual const TArray< TSharedRef<class FTokenizedMessage> >& GetFilteredMessages() const = 0;

	/** 
	 * Set the message selection state 
	 *
	 * @param	Message		The message to select/deselect.
	 * @param	bSelected	Whether to select or deselect the message.
	 */
	virtual void SelectMessage(const TSharedRef<class FTokenizedMessage>& Message, bool bSelected) = 0;
	
	/** 
	 * Get the message selection state 
	 *
	 * @param	Message		The message to check.
	 * @returns true if the message is selected.
	 */
	virtual bool IsMessageSelected(const TSharedRef<class FTokenizedMessage>& Message) const = 0;

	/** Clears the message selection */
	virtual void ClearSelectedMessages() = 0;

	/** Inverts the message selection */
	virtual void InvertSelectedMessages() = 0;

	/** 
	 * Gets all the unfiltered selected messages as a single piece of text
	 *
	 * @returns the selected messages as a string
	 */
	virtual FString GetSelectedMessagesAsString() const = 0;

	/** 
	 * Gets all the unfiltered messages as a single piece of text
	 *
	 * @returns all the messages as a string
	 */
	virtual FString GetAllMessagesAsString() const = 0;

	/** 
	 * Gets the message log listing unique name 
	 *
	 * @returns the name of the listing.
	 */
	virtual const FName& GetName() const = 0;

	/** 
	 * Sets the message log listing label
	 * 
	 * @param	InLogLabel	The label to display for this listing.
	 */
	virtual void SetLabel( const FText& InLogLabel ) = 0;

	/** 
	 * Gets the message log listing label
	 *
	 * @returns the label for this listing.
	 */
	virtual const FText& GetLabel() const = 0;

	/** 
	 * Gets the set of message filters used when displaying messages
	 * 
	 * @returns the array of message filters.
	 */
	virtual const TArray< TSharedRef< class FMessageFilter> >& GetMessageFilters() const = 0;

	/** 
	 * Performs an operation depending on the token (execute hyperlink etc.)
	 * 
	 * @param	Token	The token to execute.
	 */
	virtual void ExecuteToken( const TSharedRef<class IMessageToken>& Token ) const = 0;

	/** Broadcasts when a token is clicked/executed  */
	DECLARE_EVENT_OneParam( IMessageLogListing, IMessageTokenClickedEvent, const TSharedRef<class IMessageToken>& );
	virtual IMessageTokenClickedEvent& OnMessageTokenClicked() = 0;

	/** Broadcasts whenever we are informed of a change in the IMessageLogListing */
	DECLARE_EVENT( IMessageLogListing, FChangedEvent )
	virtual FChangedEvent& OnDataChanged() = 0;

	/** Broadcasts whenever selection state is changed */
	DECLARE_EVENT( IMessageLogListing, FOnSelectionChangedEvent )
	virtual FOnSelectionChangedEvent& OnSelectionChanged() = 0;

	/** Broadcasts whenever page selection state is changed */
	DECLARE_EVENT( IMessageLogListing, FOnPageSelectionChangedEvent )
	virtual FOnPageSelectionChangedEvent& OnPageSelectionChanged() = 0;
};
