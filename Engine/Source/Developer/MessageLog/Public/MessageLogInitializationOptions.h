// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Options for setting up a message log's UI
 */
struct FMessageLogInitializationOptions
{
	FMessageLogInitializationOptions()
		: bShowFilters(true)
		, bShowPages(false)
		, bAllowClear(true)
		, bDiscardDuplicates(false)
		, MaxPageCount(20)
		, bShowInLogWindow(true)
		, bScrollToBottom(false)
	{}

	/** Whether to show the filters menu */
	bool bShowFilters;

	/** 
	 * Whether to initially  show the pages widget. Setting this to false will allow the user to manually clear the log.
	 * If this is not set & NewPage() is called on the log, the pages widget will show itself
	 */
	bool bShowPages;

	/**
	* Whether to allow the user to clear this log.
	*/
	bool bAllowClear;

	/**
	 * Whether to check for duplicate messages & discard them
	 */
	bool bDiscardDuplicates;

	/** The maximum number of pages this log can hold. Pages are managed in a first in, last out manner */
	uint32 MaxPageCount;

	/** Whether to show this log in the main log window */
	bool bShowInLogWindow;

	/** Whether to scroll to the bottom of the window when messages are added */
	bool bScrollToBottom;
};
