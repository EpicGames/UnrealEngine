// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "MessageLogInitializationOptions.h"

class FMessageLogViewModel;
class FWorkspaceItem;

DECLARE_DELEGATE_RetVal(bool, FCanShowMessageLog);

class FMessageLogModule : public IModuleInterface
{
public:
	FMessageLogModule();

	/** Begin IModuleInterface interface */
	virtual void StartupModule();
	virtual void ShutdownModule();
	/** End IModuleInterface interface */

	/** 
	 * Registers a log listing with the message log widget. This log is displayed in the global Message Log window by default, but this can be disabled in InitializationOptions.
	 * It is not necessary to call this function before outputting to a log via AddMessage etc. This call simply
	 * registers a UI to view the log data.
	 *
	 * @param	LogName					The name of the log to register
	 * @param	LogLabel				The label to display for the log
	 * @param	InitializationOptions	Initialization options for this message log
	 */
	virtual void RegisterLogListing(const FName& LogName, const FText& LogLabel, const FMessageLogInitializationOptions& InitializationOptions = FMessageLogInitializationOptions() );

	/** 
	 * Unregisters a log listing with the message log widget.
	 *
	 * @param	LogName		The name of the log to unregister.
	 * @returns true if successful.
	 */
	virtual bool UnregisterLogListing(const FName& LogName);

	/** 
	 * Checks to see if a log listing is already registered with the system 
	 *
	 * @param	LogName		The name of the log to check.
	 * @returns true if the log listing is already registered.
	 */
	virtual bool IsRegisteredLogListing(const FName& LogName) const;

	/** 
	 * Get a message log listing registered with the message log. If it does not exist it will created.
	 *
	 * @param	LogName		The name of the log to retrieve
	 * @returns the log listing.
	 */
	virtual TSharedRef<class IMessageLogListing> GetLogListing(const FName& LogName);

	/** 
	 * Opens up the message log to a certain log listing 
	 *
	 * @param	LogName		The name of the log to open
	 */
	virtual void OpenMessageLog(const FName& LogName);

	/** 
	 * Creates a new log listing for use outside of the global MessageLog window.
	 * 
	 * @param	LogName					The name of the log to create
	 * @param	InitializationOptions	Initialization options for this message log
	 * @returns a new log listing.
	 */
	virtual TSharedRef<class IMessageLogListing> CreateLogListing(const FName& InLogName, const FMessageLogInitializationOptions& InitializationOptions = FMessageLogInitializationOptions());

	/** 
	 * Creates a log listing widget to view data from the passed-in listing.
	 * 
	 * @param	InMessageLogListing	The log listing created via CreateLogListing().
	 * @returns the log listing widget.
	 */
	virtual TSharedRef< class SWidget > CreateLogListingWidget(const TSharedRef<class IMessageLogListing>& InMessageLogListing);

	/**
	 * Setting this to true will allow the message log to be displayed when OpenMessageLog is called.
	 */
	virtual void EnableMessageLogDisplay(bool bInCanDisplayMessageLog);

	/**
	* Registers the Message Log spawner.
	* @param InGroup The Workspace group used to register the MessageLogSpawner.
	 */
	virtual void RegisterMessageLogSpawner(const TSharedRef<FWorkspaceItem>& InGroup);

private:

	/** The one and only message log view model */
	TSharedPtr<class FMessageLogViewModel> MessageLogViewModel;

	/** Delegate used to determine whether the message log can be displayed */
	bool bCanDisplayMessageLog;

	/** Handle to delegate called when module state changes */
	FDelegateHandle ModulesChangedHandle;
};
