// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/List.h"
#include "Logging/TokenizedMessage.h"

#define UE_API MESSAGELOG_API

/** Define the container type for all the messages */
typedef TArray< TSharedRef<FTokenizedMessage> > MessageContainer;

/** This class represents a set of rich tokenized messages for a particular system */
class FMessageLogListingModel : public TSharedFromThis< FMessageLogListingModel >
{
protected:
	struct FPage
	{
		FPage( const FText& InTitle )
			: Title( InTitle )
		{
		}

		/** The title of this page */
		FText Title;

		/** The list of messages in this log listing */
		MessageContainer Messages;

		TSet<uint32> MessagesHashes;
	};

public:

	virtual ~FMessageLogListingModel() = default;

	/**  
	 *	Factory method which creates a new FMessageLogListingModel object
	 *
	 *	@param	InLogName		The name of the log
	 */
	static TSharedRef< FMessageLogListingModel > Create( const FName& InLogName )
	{
		TSharedRef< FMessageLogListingModel > NewLogListing( new FMessageLogListingModel( InLogName ) );
		return NewLogListing;
	}

	/** Broadcasts whenever the message log listing changes */
	DECLARE_EVENT( FMessageLogListingModel, FChangedEvent )
	FChangedEvent& OnChanged() { return ChangedEvent; }

	/** Retrieves the name identifier for this log listing */
	const FName& GetName() const { return LogName; }

	/** Returns the message at the specified index */
	UE_API const TSharedPtr<FTokenizedMessage> GetMessageAtIndex( const uint32 PageIndex, const int32 MessageIndex ) const;

	/** Gets all messages as a string */
	UE_API FString GetAllMessagesAsString(const uint32 PageIndex) const;

	/** Obtains a const iterator to the message data structure */
	UE_API MessageContainer::TConstIterator GetMessageIterator(uint32 PageIndex) const;

	/** Replaces the message at the given index */
	UE_API int32 ReplaceMessage( const TSharedRef< FTokenizedMessage >& NewMessage, const uint32 PageIndex, const int32 MessageIndex );

	/** Appends a message */
	UE_API void AddMessage( const TSharedRef< FTokenizedMessage >& NewMessage, bool bMirrorToOutputLog = true, bool bDiscardDuplicates = false );

	/** Appends multiple messages */
	UE_API void AddMessages( const TArray< TSharedRef< FTokenizedMessage > >& NewMessages, bool bMirrorToOutputLog = true, bool bDiscardDuplicates = false );

	/** Clears all messages */
	UE_API void ClearMessages();

	/**
	 * Add a new page. Old pages are only kept around if they contain messages, so if the current
	 * page is empty, this call does nothing.
	 * @param	InTitle		The title that will be displayed for this page.
	 * @param	InMaxPages	The maximum number of pages we keep around. If the count is exceeded, 
	 *						we discard the oldest page.
	 */
	UE_API void NewPage( const FText& InTitle, uint32 InMaxPages );

	/**
	* Sets the current page to the one specified by the title. If the page does not exist, creates a new page.
	* @param	InTitle		The title of the page to search for, or create
	* @param	InMaxPages	The maximum number of pages we keep around. If the count is exceeded, 
	*						we discard the oldest page.
	* @returns true if the page has changed (either switched or created a new one).
	*/
	UE_API bool SetCurrentPage( const FText& InTitle, uint32 InMaxPages  );

	/**
	* Sets the current page to the one specified by the index. If the page does not exist, creates a new page.
	* The page will move to index 0.
	* @param	InTitle		The title of the page to search for, or create
	* @param	InMaxPages	The maximum number of pages we keep around. If the count is exceeded, 
	*						we discard the oldest page.
	* @returns true if the page has changed (either switched or created a new one).
	*/
	UE_API bool SetCurrentPage( const uint32 InOldPageIndex  );

	/** Get the number of pages contained in this log */
	UE_API uint32 NumPages() const;

	/** Get the number of messages on the passed-in page */
	UE_API uint32 NumMessages( uint32 PageIndex ) const;

	/**
	 * Get the title of the page at the specified index
	 * @param	PageIndex	The index of the page
	 */
	UE_API const FText& GetPageTitle( const uint32 PageIndex ) const;

	/** Helper function for RemoveDuplicates(), exposed so the ViewModel can use it too */
	static UE_API bool AreMessagesEqual(const TSharedRef< FTokenizedMessage >& Message0, const TSharedRef< FTokenizedMessage >& Message1);

	/** Remove any messages that are duplicates of one another - O(n) */
	UE_API void RemoveDuplicates(uint32 PageIndex);

protected:

	/** Will broadcast to all registered observers informing them of a change */
	virtual void Notify() { ChangedEvent.Broadcast(); }

	/** Access the current page (we only add messages to this page */
	UE_API FPage& CurrentPage() const;

	/** Get a page by index - uses cache to speed up linked list access */
	UE_API FPage* PageAtIndex(const uint32 PageIndex) const;

	/** Create a new page if we have one pending */
	UE_API void CreateNewPageIfRequired();

private:
	/**
	 *	FMessageLogListingModel Constructor
	 *
	 *	@param	InLogName		The name of the log
	 */
	FMessageLogListingModel( const FName& InLogName )
		: LogName( InLogName )
		, bIsPrintingToOutputLog( false )
	{
		check( LogName != NAME_None );

		// create default page
		Pages.AddTail(FPage(FText::FromName(LogName)));
		CachedPage = &Pages.GetTail()->GetValue();
		CachedPageIndex = 0;
	}

	/** Helper function for AddMessage and AddMessages */
	UE_API void AddMessageInternal( const TSharedRef<FTokenizedMessage>& NewMessage, bool bMirrorToOutputLog, bool bDiscardDuplicates );
	
private:
	/** The name of a pending page */
	FText PendingPageName;

	/** The cap on the number of pages we have */
	uint32 MaxPages;

	/** The list of pages in this log listing */
	TDoubleLinkedList<FPage> Pages;

	/** Name of the listing, for identification */
	FName LogName;

	/** Delegate to call when data is changed */
	FChangedEvent ChangedEvent;

	// Are we currently processing the output log mirror?  If so, we drop any additional messages we receive, as they are duplicates
	bool bIsPrintingToOutputLog;

	/** Cached page index */
	mutable uint32 CachedPageIndex;

	/** Cached page */
	mutable FPage* CachedPage;
};

#undef UE_API
