// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MessageLogInitializationOptions.h"
#include "Logging/TokenizedMessage.h"
#include "IMessageLogListing.h"
#include "Model/MessageLogListingModel.h"

#define UE_API MESSAGELOG_API

class FMessageFilter;

/**
 * The non-UI solution specific presentation logic for a collection of messages for a particular system.
 */
class FMessageLogListingViewModel
	: public TSharedFromThis<FMessageLogListingViewModel>
	, public IMessageLogListing
{
public:

	/**  
	 *	Factory method which creates a new FMessageLogListingViewModel object
	 *
	 *	@param	InMessageLogListingModel		The MessageLogListing data to view
	 *	@param	InLogLabel						The label that will be displayed in the UI for this log listing
	 *	@param	InMessageLogListingModel		If true, a filters list will be displayed for this log listing
	 */
	static TSharedRef< FMessageLogListingViewModel > Create( const TSharedRef< FMessageLogListingModel >& InMessageLogListingModel, const FText& InLogLabel, const FMessageLogInitializationOptions& InitializationOptions = FMessageLogInitializationOptions() )
	{
		TSharedRef< FMessageLogListingViewModel > NewLogListingView( new FMessageLogListingViewModel( InMessageLogListingModel, InLogLabel, InitializationOptions ) );
		NewLogListingView->Initialize();

		return NewLogListingView;
	}

public:
	UE_API FMessageLogListingViewModel();
	UE_API ~FMessageLogListingViewModel();

	/** Begin IMessageLogListing interface */
	UE_API virtual void AddMessage( const TSharedRef< class FTokenizedMessage >& NewMessage, bool bMirrorToOutputLog ) override;
	UE_API virtual void AddMessages( const TArray< TSharedRef< class FTokenizedMessage > >& NewMessages, bool bMirrorToOutputLog ) override;
	UE_API virtual void ClearMessages() override;
	UE_API virtual const TArray< TSharedRef<class FTokenizedMessage> >& GetSelectedMessages() const override;
	UE_API virtual void SelectMessages( const TArray< TSharedRef<class FTokenizedMessage> >& InSelectedMessages ) override;
	UE_API virtual const TArray< TSharedRef<class FTokenizedMessage> >& GetFilteredMessages() const override;
	UE_API virtual void SelectMessage(const TSharedRef<class FTokenizedMessage>& Message, bool bSelected) override;
	UE_API virtual bool IsMessageSelected(const TSharedRef<class FTokenizedMessage>& Message) const override;
	UE_API virtual void ClearSelectedMessages() override;
	UE_API virtual void InvertSelectedMessages() override;
	UE_API virtual FString GetSelectedMessagesAsString() const override;
	UE_API virtual FString GetAllMessagesAsString() const override;
	UE_API virtual const FName& GetName() const override;
	UE_API virtual void SetLabel( const FText& InLogLabel ) override;
	UE_API virtual const FText& GetLabel() const override;
	UE_API virtual const TArray< TSharedRef<FMessageFilter> >& GetMessageFilters() const override;
	UE_API virtual void ExecuteToken( const TSharedRef<class IMessageToken>& Token ) const override;
	UE_API virtual void NewPage( const FText& Title ) override;
	UE_API virtual void SetCurrentPage( const FText& Title ) override;
	UE_API virtual void SetCurrentPage( const uint32 InOldPageIndex ) override;
	UE_API virtual void NotifyIfAnyMessages( const FText& Message, EMessageSeverity::Type SeverityFilter = EMessageSeverity::Info, bool bForce = false ) override;
	UE_API virtual void Open() override;
	UE_API virtual int32 NumMessages( EMessageSeverity::Type SeverityFilter ) override;

	DECLARE_DERIVED_EVENT(FMessageLogListingViewModel, IMessageLogListing::IMessageTokenClickedEvent, IMessageTokenClickedEvent)
	virtual IMessageLogListing::IMessageTokenClickedEvent& OnMessageTokenClicked() override { return TokenClickedEvent; }

	DECLARE_DERIVED_EVENT(FMessageLogListingViewModel, IMessageLogListing::FChangedEvent, FChangedEvent)
	virtual IMessageLogListing::FChangedEvent& OnDataChanged() override { return ChangedEvent; }

	DECLARE_DERIVED_EVENT(FMessageLogListingViewModel, IMessageLogListing::FOnSelectionChangedEvent, FOnSelectionChangedEvent)
	virtual IMessageLogListing::FOnSelectionChangedEvent& OnSelectionChanged() override { return SelectionChangedEvent; }

	DECLARE_DERIVED_EVENT(FMessageLogListingViewModel, IMessageLogListing::FOnPageSelectionChangedEvent, FOnPageSelectionChangedEvent )
	virtual IMessageLogListing::FOnPageSelectionChangedEvent& OnPageSelectionChanged() override { return PageSelectionChangedEvent; }
	/** End IMessageLogListing interface */

	/** Initializes the FMessageLogListingViewModel for use */
	UE_API virtual void Initialize();

	/** Handles updating the viewmodel when one of its filters changes */
	UE_API void OnFilterChanged();

	/** Called when data is changed changed/updated in the model */
	UE_API virtual void OnChanged();

	/** Returns the filtered message list */
	const TArray< TSharedRef< FTokenizedMessage > >& GetFilteredMessages() { return FilteredMessages; }

	/** Obtains a const iterator to the filtered messages */
	UE_API MessageContainer::TConstIterator GetFilteredMessageIterator() const;

	/** Obtains a const iterator to the selected filtered messages list */
	UE_API MessageContainer::TConstIterator GetSelectedMessageIterator() const;

	/** Returns the message at the specified index in the filtered list */
	UE_API const TSharedPtr<FTokenizedMessage> GetMessageAtIndex( const int32 MessageIndex ) const;

	/** Set whether we should show filters or not */
	UE_API void SetShowFilters(bool bInShowFilters);

	/** Get whether we should show filters or not */
	UE_API bool GetShowFilters() const;

	/** Set whether we should show pages or not */
	UE_API void SetShowPages(bool bInShowPages);

	/** Get whether we should show pages or not */
	UE_API bool GetShowPages() const;

	/** Set whether we should scroll to the bottom when messages are added */
	UE_API void SetScrollToBottom(bool bInScrollToBottom);

	/** Get whether we should scroll to the bottom when messages are added */
	UE_API bool GetScrollToBottom() const;

	/** Set whether we should show allow the user to clear the log. */
	UE_API void SetAllowClear(bool bInAllowClear);

	/** Get whether we should show allow the user to clear the log. */
	UE_API bool GetAllowClear() const;

	/** Set whether we should discard duplicates or not */
	UE_API void SetDiscardDuplicates(bool bInDiscardDuplicates);

	/** Get whether we should discard duplicates or not */
	UE_API bool GetDiscardDuplicates() const;

	/** Set the maximum page count this log can hold */
	UE_API void SetMaxPageCount(uint32 InMaxPageCount);

	/** Get the maximum page count this log can hold */
	UE_API uint32 GetMaxPageCount() const;
	
	/** Get the number of pages we can view */
	UE_API uint32 GetPageCount() const;

	/** Get the current page index we are viewing */
	UE_API uint32 GetCurrentPageIndex() const;

	/** Set the current page index we are viewing */
	UE_API void SetCurrentPageIndex( uint32 InCurrentPageIndex );

	/** Go to the page at the index after the current */
	UE_API void NextPage();

	/** Go to the page at the index before the current */
	UE_API void PrevPage();

	/**
	 * Get the title of the page at the specified index
	 * @param	PageIndex	The index of the page
	 */
	UE_API const FText& GetPageTitle( const uint32 PageIndex ) const;

	/** Gets the number of messages in the current log page */
	UE_API uint32 NumMessages() const;

	/** Get whether to show this log in the main log window */
	bool ShouldShowInLogWindow() const { return bShowInLogWindow; }

private:
	FMessageLogListingViewModel( TSharedPtr< FMessageLogListingModel > InMessageLogListingModel, const FText& InLogLabel, const FMessageLogInitializationOptions& InitializationOptions )
		: bShowFilters( InitializationOptions.bShowFilters )
		, bShowPages( InitializationOptions.bShowPages )
		, bAllowClear( InitializationOptions.bAllowClear )
		, bDiscardDuplicates( InitializationOptions.bDiscardDuplicates )
		, bScrollToBottom( InitializationOptions.bScrollToBottom )
		, MaxPageCount( InitializationOptions.MaxPageCount )
		, bShowInLogWindow( InitializationOptions.bShowInLogWindow )
		, CurrentPageIndex( 0 )
		, bIsRefreshing( false )
		, LogLabel( InLogLabel )
		, MessageLogListingModel( InMessageLogListingModel )
	{}

	/** Rebuilds the list of filtered messages */
	UE_API void RefreshFilteredMessages();

	/** Dismisses a notification that was previously shown. */
	UE_API void DismissNotification(int32 NotificationId);

	/** Opens the message log from a notification that was previously shown. */
	UE_API void OpenMessageLogFromNotification(int32 NotificationId);

	/** Helper function for opening this message log from a notification */
	UE_API void OpenMessageLog();

	/** Helper function to check if this log contains messages above a certain severity */
	UE_API int32 NumMessagesPresent( uint32 PageIndex, EMessageSeverity::Type InSeverity ) const;

	/** Helper function to check the worst severity contained in this log page */
	UE_API EMessageSeverity::Type HighestSeverityPresent( uint32 PageIndex ) const;

private:

	/** Whether filters should be shown for this listing */
	bool bShowFilters;

	/** Whether pages should be used/shown for this listing */
	bool bShowPages;

	/** Whether we allow the user to clear the log. */
	bool bAllowClear;

	/** Whether to check for duplicate messages & discard them */
	bool bDiscardDuplicates;

	/** Whether to scroll to the bottom when messages are added */
	bool bScrollToBottom;

	/** The limit on the number of displayed pages for this listing */
	uint32 MaxPageCount;

	/** Whether to show this log in the main log window */
	bool bShowInLogWindow;

	/** The currently displayed page index */
	uint32 CurrentPageIndex;

	/** Tracks if the viewmodel is in the middle of refreshing */
	bool bIsRefreshing;

	/** Label of the listing, displayed to users */
	FText LogLabel;

	/* The model we are getting display info from */
	TSharedPtr< FMessageLogListingModel > MessageLogListingModel;

	/** The same list of messages in the model after filtering is applied */
	MessageContainer FilteredMessages;

	/** The list of selected messages */
	MessageContainer SelectedFilteredMessages;

	/** The array of message filters used on this listing */
	TArray< TSharedRef< FMessageFilter > > MessageFilters;

	/** Delegate to call when a token is clicked */
	IMessageTokenClickedEvent TokenClickedEvent;

	/** Delegate to call when model data is changed */
	FChangedEvent ChangedEvent;

	/** Delegate to call when selection state is changed */
	FOnSelectionChangedEvent SelectionChangedEvent;

	/** Delegate to call when page selection state is changed */
	FOnPageSelectionChangedEvent PageSelectionChangedEvent;

	/** All open notifications */
	struct FOpenNotification
	{
		int32 NotificationId;
		TWeakPtr<class SNotificationItem> NotificationItem;
		const FText NotificationMessage;

		FOpenNotification(int32 InNotificationId,
		                  const TWeakPtr<class SNotificationItem>& InNotificationItem,
		                  const FText InNotificationMessage)
			: NotificationId(InNotificationId)
			, NotificationItem(InNotificationItem)
			, NotificationMessage( InNotificationMessage)
		{
		}
		FOpenNotification() : NotificationId(INDEX_NONE) {}
	};
	TArray<FOpenNotification> OpenNotifications;
	static UE_API int32 NextNotificationId;
};

#undef UE_API
