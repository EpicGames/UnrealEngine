// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/TokenizedMessage.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Presentation/MessageLogListingViewModel.h"
#include "Framework/Commands/UICommandList.h"

#define UE_API MESSAGELOG_API

class FMessageFilter;

/**
 * A message log listing, such as the Compiler Log, or the Map Check Log.
 * Holds the log lines, and any extra widgets necessary.
 */
class SMessageLogListing : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMessageLogListing)
		{}

	SLATE_END_ARGS()

	UE_API SMessageLogListing();

	UE_API ~SMessageLogListing();

	UE_API void Construct( const FArguments& InArgs, const TSharedRef< class IMessageLogListing >& InModelView );

	/** Refreshes the log messages, reapplying any filtering */
	UE_API void RefreshVisibility();

	/** Used to execute the 'on clicked token' delegate */
	UE_API void BroadcastMessageTokenClicked( TSharedPtr< class FTokenizedMessage > Message, const TSharedRef<class IMessageToken>& Token );
	/** Used to execute a message's featured action (a token associated with the entire message) */
	UE_API void BroadcastMessageDoubleClicked(TSharedPtr< class FTokenizedMessage > Message);

	/** Gets a list of the selected messages */
	UE_API const TArray< TSharedRef< class FTokenizedMessage > > GetSelectedMessages() const;

	/** Set the message selection state */
	UE_API void SelectMessage( const TSharedRef< class FTokenizedMessage >& Message, bool bSelected ) const;

	/** Get the message selection state */
	UE_API bool IsMessageSelected( const TSharedRef< class FTokenizedMessage >& Message ) const;

	/** Scrolls the message into view */
	UE_API void ScrollToMessage( const TSharedRef< class FTokenizedMessage >& Message ) const;

	/** Clears the message selection */
	UE_API void ClearSelectedMessages() const;

	/** Inverts the message selection */
	UE_API void InvertSelectedMessages() const;

	/** Compiles the selected messages into a single string. */
	UE_API FString GetSelectedMessagesAsString() const;

	/** Compiles all the messages into a single string. */
	UE_API FString GetAllMessagesAsString() const;

	/** Gets the message log listing unique name */
	const FName& GetName() const { return MessageLogListingViewModel->GetName(); }

	/** Gets the message log listing label  */
	const FText& GetLabel() const { return MessageLogListingViewModel->GetLabel(); }

	/** Gets the set of message filters used when displaying messages */
	const TArray< TSharedRef< class FMessageFilter > >& GetMessageFilters() const { return MessageLogListingViewModel->GetMessageFilters(); }

	/** Called when data is changed changed/updated in the model */
	UE_API void OnChanged();

	/** Called whenever the viewmodel selection changes */
	UE_API void OnSelectionChanged();

	/**
	 * Copies the selected messages to the clipboard
	 */
	UE_API void CopySelectedToClipboard() const;

	/**
	 * Copies log output (selected/all), optionally to the clipboard
	 */
	UE_API FString CopyLogOutput( bool bSelected, bool bClipboard ) const;

	/** @return	The UICommandList supported by the MessageLogView */
	UE_API const TSharedRef< const FUICommandList > GetCommandList() const;

	/**
	 * Called after a key is pressed when this widget has focus
	 *
	 * @param  InKeyEvent  Key event
	 *
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	UE_API FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	/** Delegate supplying a label for the page-flipper widget */
	UE_API FString GetPageText() const;

	/** Delegate to display the previous page */
	UE_API FReply OnClickedPrevPage();

	/** Delegate to display the next page */
	UE_API FReply OnClickedNextPage();

	/** Delegate to get the display visibility of the show filter combobox */
	UE_API EVisibility GetFilterMenuVisibility();

	/** Generates the widgets for show filtering */
	UE_API TSharedRef<ITableRow> MakeShowWidget(TSharedRef<FMessageFilter> Selection, const TSharedRef<STableViewBase>& OwnerTable);

	/** Generates the menu content for filtering log content */
	UE_API TSharedRef<SWidget> OnGetFilterMenuContent();

	/** Delegate for enabling & disabling the page widget depending on the number of pages */
	UE_API bool IsPageWidgetEnabled() const;

	/** Delegate for showing & hiding the page widget depending on whether this log uses pages or not */
	UE_API EVisibility GetPageWidgetVisibility() const;

	/** Delegate for enabling & disabling the clear button depending on the number of messages */
	UE_API bool IsClearWidgetEnabled() const;

	/** Delegate for showing & hiding the clear button depending on whether this log uses pages or not */
	UE_API EVisibility GetClearWidgetVisibility() const;

	/** Delegate to generate the label for the page combobox */
	UE_API FText OnGetPageMenuLabel() const;

	/** Delegate to generate the menu content for the page combobox */
	UE_API TSharedRef<SWidget> OnGetPageMenuContent() const;

	/** Delegate called when a page is selected from the page menu */
	UE_API void OnPageSelected(uint32 PageIndex);

	/** Delegate for Clear button */
	UE_API FReply OnClear();

private:
	/** Makes the message log line widgets for populating the listing */
	UE_API TSharedRef<ITableRow> MakeMessageLogListItemWidget( TSharedRef< class FTokenizedMessage > Message, const TSharedRef< STableViewBase >& OwnerTable );
	
	/** Called when map check message line selection has changed */
	UE_API void OnLineSelectionChanged( TSharedPtr< class FTokenizedMessage > Selection, ESelectInfo::Type SelectInfo );

	void ContextMenuSelect() const;
	void ContextMenuSelectByMessage(const TCHAR* FormatKey) const;
	TSharedPtr<SWidget> ContextMenuOpening();

private:
	/** The list of commands with bound delegates for the message log */
	const TSharedRef< FUICommandList > UICommandList;

	/** Reference to the ViewModel which holds state info and has access to data */
	TSharedPtr<class FMessageLogListingViewModel> MessageLogListingViewModel;

	/**	Whether the view is currently updating the viewmodel selection */
	bool bUpdatingSelection;

	/** The list view for showing all the message log lines */
	TSharedPtr< SListView< TSharedRef< class FTokenizedMessage > > > MessageListView;
};

#undef UE_API
